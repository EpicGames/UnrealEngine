[/Script/WaveformEditorWidgets.WaveformEditorWidgetsSettings]
PlayheadColor=(R=1.000000,G=0.100000,B=0.200000,A=1.000000)
WaveformColor=(R=1.000000,G=1.000000,B=1.000000,A=1.000000)
WaveformBackgroundColor=(R=0.017642,G=0.017642,B=0.017642,A=1.000000)
MajorGridColor=(R=0.000000,G=0.000000,B=0.000000,A=1.000000)
MinorGridColor=(R=0.000000,G=0.000000,B=0.000000,A=0.500000)
RulerBackgroundColor=(R=0.000000,G=0.000000,B=0.000000,A=1.000000)
RulerTicksColor=(R=1.000000,G=1.000000,B=1.000000,A=0.900000)
RulerTextColor=(R=1.000000,G=1.000000,B=1.000000,A=0.900000)
RulerFontSize=10.000000


[CoreRedirects]
+PropertyRedirects=(OldName="/Script/WaveformTransformations.WaveCueArray.WaveCues",NewName="/Script/WaveformTransformations.WaveCueArray.CuesAndLoops")
