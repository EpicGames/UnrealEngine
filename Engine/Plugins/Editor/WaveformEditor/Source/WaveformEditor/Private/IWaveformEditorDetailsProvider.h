// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "UObject/ObjectPtr.h"

class IPropertyHandle;

class IWaveformEditorDetailsProvider
{
public:
	virtual void GetHandlesForUObjectProperties(const TObjectPtr<UObject> InUObject, TArray<TSharedRef<IPropertyHandle>>& OutPropertyHandles) = 0;
};