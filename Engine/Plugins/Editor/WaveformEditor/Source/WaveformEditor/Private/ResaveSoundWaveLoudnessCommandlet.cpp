// Copyright Epic Games, Inc. All Rights Reserved.

#include "ResaveSoundWaveLoudnessCommandlet.h"

#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/PackageName.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundSourceBus.h"
#include "Sound/SoundWaveProcedural.h"
#include "WaveformEditorLog.h"
#include "WaveformEditorModule.h"

UResaveSoundWaveLoudnessCommandlet::UResaveSoundWaveLoudnessCommandlet()
{
	// Shown in help text when running -run=help
	HelpDescription = TEXT("Migrates legacy SoundWave assets by computing missing loudness metadata (LUFS and SamplePeakDB).");
	HelpUsage = TEXT("UnrealEditor.exe Project.uproject -run=ResaveSoundWaveLoudness [-PACKAGEFOLDER=/Game/Audio] [-AutoCheckOut] [-AutoCheckIn]");

	// We only care about saving packages that we actually modify.
	bOnlySaveDirtyPackages = true;
}

int32 UResaveSoundWaveLoudnessCommandlet::Main(const FString& Params)
{
	// Run the base ResavePackages loop, which calls PerformAdditionalOperations for each object.
	const int32 Result = Super::Main(Params);

	// Report migration summary.
	UE_LOGF(LogWaveformEditor, Display, "");
	UE_LOGF(LogWaveformEditor, Display, "=== SoundWave Loudness Migration Summary ===");
	UE_LOGF(LogWaveformEditor, Display, "  Migrated : %d (loudness computed and package resaved)", NumMigrated);
	UE_LOGF(LogWaveformEditor, Display, "  Skipped  : %d (already had loudness data)", NumSkipped);
	UE_LOGF(LogWaveformEditor, Display, "  Failed   : %d (could not read raw PCM data)", NumFailed);
	UE_LOGF(LogWaveformEditor, Display, "  Total    : %d", NumMigrated + NumSkipped + NumFailed);
	UE_LOGF(LogWaveformEditor, Display, "=============================================");

	return Result;
}

bool UResaveSoundWaveLoudnessCommandlet::ShouldSkipPackage(const FString& Filename)
{
	if (Super::ShouldSkipPackage(Filename))
	{
		return true;
	}

	FString PackageName;
	if (!FPackageName::TryConvertFilenameToLongPackageName(Filename, PackageName))
	{
		return true;
	}

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();

	TArray<FAssetData> Assets;
	AssetRegistry.GetAssetsByPackageName(FName(*PackageName), Assets, true);

	for (const FAssetData& Asset : Assets)
	{
		if (Asset.IsInstanceOf(USoundWave::StaticClass())
			&& !Asset.IsInstanceOf(USoundWaveProcedural::StaticClass())
			&& !Asset.IsInstanceOf(USoundSourceBus::StaticClass()))
		{
			return false;
		}
	}

	return true;
}

void UResaveSoundWaveLoudnessCommandlet::PerformAdditionalOperations(UObject* Object, bool& bSavePackage)
{
	Super::PerformAdditionalOperations(Object, bSavePackage);

	USoundWave* SoundWave = Cast<USoundWave>(Object);

	if (!SoundWave)
	{
		return;
	}

	// Filtered out by UResaveSoundWaveLoudnessCommandlet::ShouldSkipPackage
	check(!SoundWave->IsA<USoundWaveProcedural>() && !SoundWave->IsA<USoundSourceBus>());

	// Only migrate assets that are missing loudness data.
	if (SoundWave->LUFS != 0.f || SoundWave->SamplePeakDB != 0.f)
	{
		NumSkipped++;
		return;
	}

	// Compute and apply loudness values using the shared analysis function.
	if (!WaveformAnalysis::AnalyzeSoundWaveLoudness(SoundWave))
	{
		UE_LOGF(LogWaveformEditor, Warning, "  FAILED: '%ls' - could not analyze raw PCM data", *SoundWave->GetPathName());
		NumFailed++;
		return;
	}

	bSavePackage = true;

	UE_LOGF(LogWaveformEditor, Display, "  Migrated: '%ls' (LUFS=%.1f, PeakDB=%.1f)", *SoundWave->GetPathName(), SoundWave->LUFS, SoundWave->SamplePeakDB);
	NumMigrated++;
}
