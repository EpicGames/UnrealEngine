// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Commandlets/ResavePackagesCommandlet.h"

#include "ResaveSoundWaveLoudnessCommandlet.generated.h"

/**
 * Commandlet to migrate legacy USoundWave assets that are missing loudness metadata
 * (LUFS and SamplePeakDB). These values were not computed on import prior to 5.8.
 *
 * Inherits from UResavePackagesCommandlet, so all standard path filtering and source
 * control options are available:
 *
 *   # Migrate all SoundWaves in the project
 *   UnrealEditor.exe Project.uproject -run=ResaveSoundWaveLoudness -AutoCheckOut
 *
 *   # Scope to specific content folders (+ delimited for multiple paths)
 *   UnrealEditor.exe Project.uproject -run=ResaveSoundWaveLoudness -PACKAGEFOLDER=/Game/Audio+/Game/SFX -AutoCheckOut
 *
 *   # With batched source control and auto-checkin
 *   UnrealEditor.exe Project.uproject -run=ResaveSoundWaveLoudness -AutoCheckOut -AutoCheckIn -BatchSourceControl
 */
UCLASS()
class UResaveSoundWaveLoudnessCommandlet : public UResavePackagesCommandlet
{
	GENERATED_BODY()

public:
	UResaveSoundWaveLoudnessCommandlet();

	// UCommandlet interface
	virtual int32 Main(const FString& Params) override;

protected:
	// UResavePackagesCommandlet interface
	virtual bool ShouldSkipPackage(const FString& Filename) override;
	virtual void PerformAdditionalOperations(UObject* Object, bool& bSavePackage) override;

private:
	/** Number of SoundWaves that had loudness data computed and will be resaved. */
	int32 NumMigrated = 0;

	/** Number of SoundWaves that already had valid loudness data. */
	int32 NumSkipped = 0;

	/** Number of SoundWaves where loudness analysis failed (no raw PCM data). */
	int32 NumFailed = 0;
};
