// Copyright Epic Games, Inc. All Rights Reserved.

#include "TransformedWaveformViewFactory.h"

#include "Sound/SoundWave.h"
#include "SparseSampledSequenceTransportCoordinator.h"
#include "SWaveformTransformationsOverlay.h"
#include "TransformedWaveformView.h"
#include "WaveformEditorDetailsCustomization.h"
#include "WaveformEditorSequenceDataProvider.h"
#include "WaveformEditorZoomController.h"

static FLazyName TimeRulerWidgetName("SFixedSampledSequenceRuler");

FTransformedWaveformViewFactory& FTransformedWaveformViewFactory::Get()
{
	check(Instance);
	return *Instance;
}

void FTransformedWaveformViewFactory::Create()
{
	if (Instance == nullptr)
	{
		Instance = MakeUnique<FTransformedWaveformViewFactory>();
	}
}

FTransformedWaveformView FTransformedWaveformViewFactory::GetTransformedView(TObjectPtr<USoundWave> SoundWaveToView, TSharedRef<FSparseSampledSequenceTransportCoordinator> TransportCoordinator, FNotifyHook* OwnerNotifyHook /*= nullptr*/, TSharedPtr<FWaveformEditorZoomController> InZoomController /*= nullptr*/)
{
	TSharedPtr<FWaveformEditorSequenceDataProvider> WaveformDataProvider = MakeShared<FWaveformEditorSequenceDataProvider>(SoundWaveToView);
	WaveformDataProvider->UpdateRenderElements();

	FFixedSampledSequenceView SequenceView = WaveformDataProvider->RequestSequenceView(TRange<double>::Inclusive(0, 1));

	TFunction<float(const float)> TransformationsOverlayZoomConverter = [TransportCoordinator](const float InAnchorRatio)
	{
		return TransportCoordinator->ConvertAbsoluteRatioToZoomed(InAnchorRatio);
	};

	TSharedPtr<SWaveformTransformationsOverlay> TransformationsOverlay = SNew(SWaveformTransformationsOverlay, WaveformDataProvider->GetTransformLayers(), InZoomController).AnchorsRatioConverter(TransformationsOverlayZoomConverter);

	TSharedPtr<STransformedWaveformViewPanel> WaveformPanel = SNew(STransformedWaveformViewPanel, SequenceView).TransformationsOverlay(TransformationsOverlay).ZoomController(InZoomController);
	SetUpWaveformPanelInteractions(WaveformPanel.ToSharedRef(), TransportCoordinator, InZoomController, TransformationsOverlay);

	WaveformDataProvider->OnLayersChainGenerated.AddSP(TransformationsOverlay.Get(), &SWaveformTransformationsOverlay::OnLayerChainGenerated);
	WaveformDataProvider->OnRenderElementsUpdated.AddSP(TransformationsOverlay.Get(), &SWaveformTransformationsOverlay::UpdateLayerConstraints);
	WaveformDataProvider->OnDataViewGenerated.AddSP(WaveformPanel.Get(), &STransformedWaveformViewPanel::ReceiveSequenceView);

	return FTransformedWaveformView{ WaveformPanel, WaveformDataProvider};

}

void FTransformedWaveformViewFactory::SetUpWaveformPanelInteractions(TSharedRef<STransformedWaveformViewPanel> WaveformPanel, TSharedRef<FSparseSampledSequenceTransportCoordinator> TransportCoordinator, TSharedPtr<FWaveformEditorZoomController> InZoomController /*= nullptr*/, TSharedPtr<SWaveformTransformationsOverlay> InTransformationsOverlay /*=nullptr*/)
{
	FPointerEventHandler HandlePlayheadOverlayMouseButtonUp = FPointerEventHandler::CreateLambda([TransportCoordinator](const FGeometry& Geometry, const FPointerEvent& MouseEvent)
	{
		const bool HandleLeftMouseButton = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
		if (HandleLeftMouseButton)
		{
			const float LocalWidth = Geometry.GetLocalSize().X;

			if (LocalWidth > 0.f)
			{

				const float NewPosition = Geometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition()).X / LocalWidth;
				TransportCoordinator->ScrubFocusPoint(NewPosition, false);
			}
		}

		return FReply::Handled();
	});

	FPointerEventHandler HandleTimeRulerMouseButtonUp = FPointerEventHandler::CreateLambda([WaveformPanel, TransportCoordinator, this](const FGeometry& Geometry, const FPointerEvent& MouseEvent)
	{
		TSharedRef<SWidget> WavePanelChildSlot = WaveformPanel->GetChildren()->GetChildAt(0);

		const bool HandleRightMouseButton = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton;
		
		if (HandleRightMouseButton)
		{
			return WaveformPanel->LaunchTimeRulerContextMenu();
		}

		for (int32 ChildIndex = 0; ChildIndex < WavePanelChildSlot->GetChildren()->Num(); ++ChildIndex)
		{
			TSharedRef<SWidget> RulerWidget = WavePanelChildSlot->GetChildren()->GetChildAt(ChildIndex);

			if (RulerWidget->GetType() == TimeRulerWidgetName)
			{
				return HandleTimeRulerInteraction(TransformedWaveformViewFactory::EReceivedInteractionType::MouseButtonUp, TransportCoordinator, RulerWidget, MouseEvent, Geometry);
			}
		}

		return FReply::Unhandled();
	});

	FPointerEventHandler HandleTimeRulerMouseButtonDown = FPointerEventHandler::CreateLambda([WaveformPanel, TransportCoordinator, this](const FGeometry& Geometry, const FPointerEvent& MouseEvent)
	{
		TSharedRef<SWidget> WavePanelChildSlot = WaveformPanel->GetChildren()->GetChildAt(0);

		for (int32 ChildIndex = 0; ChildIndex < WavePanelChildSlot->GetChildren()->Num(); ++ChildIndex)
		{
			TSharedRef<SWidget> RulerWidget = WavePanelChildSlot->GetChildren()->GetChildAt(ChildIndex);
			if (RulerWidget->GetType() == TimeRulerWidgetName)
			{
				return HandleTimeRulerInteraction(TransformedWaveformViewFactory::EReceivedInteractionType::MouseButtonDown, TransportCoordinator, RulerWidget, MouseEvent, Geometry);
			}
		}

		return FReply::Unhandled();
	});

	FPointerEventHandler HandleTimeRulerMouseMove = FPointerEventHandler::CreateLambda([WaveformPanel, TransportCoordinator, this](const FGeometry& Geometry, const FPointerEvent& MouseEvent)
	{
		TSharedRef<SWidget> WavePanelChildSlot = WaveformPanel->GetChildren()->GetChildAt(0);

		for (int32 ChildIndex = 0; ChildIndex < WavePanelChildSlot->GetChildren()->Num(); ++ChildIndex)
		{
			TSharedRef<SWidget> RulerWidget = WavePanelChildSlot->GetChildren()->GetChildAt(ChildIndex);
			if (RulerWidget->GetType() == TimeRulerWidgetName)
			{
				return HandleTimeRulerInteraction(TransformedWaveformViewFactory::EReceivedInteractionType::MouseMove, TransportCoordinator, RulerWidget, MouseEvent, Geometry);
			}
		}

		return FReply::Unhandled();

	});

	
	FPointerEventHandler HandleMouseWheel = FPointerEventHandler::CreateLambda([InZoomController , InTransformationsOverlay](const FGeometry& Geometry, const FPointerEvent& MouseEvent)
	{
		if (InTransformationsOverlay)
		{
			FReply TransformationsOverlayReply = InTransformationsOverlay->OnMouseWheel(Geometry, MouseEvent);

			if (TransformationsOverlayReply.IsEventHandled())
			{
				return TransformationsOverlayReply;
			}
		}

		if (InZoomController)
		{
			InZoomController->ZoomByDelta(MouseEvent.GetWheelDelta());
			return FReply::Handled();
		}

		return FReply::Unhandled();

	});

	WaveformPanel->SetOnMouseWheel(HandleMouseWheel);


	WaveformPanel->SetOnPlayheadOverlayMouseButtonUp(HandlePlayheadOverlayMouseButtonUp);
	WaveformPanel->SetOnTimeRulerMouseButtonUp(HandleTimeRulerMouseButtonUp);
	WaveformPanel->SetOnTimeRulerMouseButtonDown(HandleTimeRulerMouseButtonDown);
	WaveformPanel->SetOnTimeRulerMouseMove(HandleTimeRulerMouseMove);
}

FReply FTransformedWaveformViewFactory::HandleTimeRulerInteraction(const TransformedWaveformViewFactory::EReceivedInteractionType MouseInteractionType, TSharedRef<FSparseSampledSequenceTransportCoordinator> TransportCoordinator, const TSharedRef<SWidget> TimeRulerWidget, const FPointerEvent& MouseEvent, const FGeometry& Geometry)
{

	const float LocalWidth = Geometry.GetLocalSize().X;

	if (LocalWidth > 0.f)
	{
		const FVector2D ScreenSpacePosition = MouseEvent.GetScreenSpacePosition();
		const FVector2D CursorPosition = Geometry.AbsoluteToLocal(ScreenSpacePosition);
		const float CursorXRatio = CursorPosition.X / LocalWidth;

		switch (MouseInteractionType)
		{
		case TransformedWaveformViewFactory::EReceivedInteractionType::MouseButtonDown:
			if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
			{
				return FReply::Handled().CaptureMouse(TimeRulerWidget).PreventThrottling();
			}
			break;
		case TransformedWaveformViewFactory::EReceivedInteractionType::MouseMove:
			if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton))
			{
				TransportCoordinator->ScrubFocusPoint(CursorXRatio, true);
				return FReply::Handled().CaptureMouse(TimeRulerWidget);
			}
			break;
		case TransformedWaveformViewFactory::EReceivedInteractionType::MouseButtonUp:
			if (TimeRulerWidget->HasMouseCapture())
			{
				TransportCoordinator->ScrubFocusPoint(CursorXRatio, false);
				return FReply::Handled().ReleaseMouseCapture();
			}
			break;
		default:
			static_assert(static_cast<int32>(TransformedWaveformViewFactory::EReceivedInteractionType::COUNT) == 3, "Possible missing switch case coverage for 'EReceivedInteractionType'");
			break;
		}
	}

	return FReply::Handled();
}

TUniquePtr<FTransformedWaveformViewFactory> FTransformedWaveformViewFactory::Instance;
