// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaveformEditorSequenceDataProvider.h"
#include "Input/Reply.h"

#define UE_API WAVEFORMEDITOR_API

class FNotifyHook;
class FSparseSampledSequenceTransportCoordinator;
class FWaveformEditorZoomController; 
class IDetailsView;
class STransformedWaveformViewPanel;
class SWaveformTransformationsOverlay;
class SWidget;
class USoundWave;
struct FGeometry;
struct FPointerEvent;
struct FTransformedWaveformView;

namespace TransformedWaveformViewFactory
{
	enum class EReceivedInteractionType
	{
		MouseButtonUp,
		MouseButtonDown,
		MouseMove,
		COUNT
	};
}

class FTransformedWaveformViewFactory
{	
public:
	static UE_API FTransformedWaveformViewFactory& Get();
	static UE_API void Create();

	UE_API FTransformedWaveformView GetTransformedView(TObjectPtr<USoundWave> SoundWaveToView, TSharedRef<FSparseSampledSequenceTransportCoordinator> TransportCoordinator, FNotifyHook* OwnerNotifyHook = nullptr, TSharedPtr<FWaveformEditorZoomController> InZoomController = nullptr);

private:
	UE_API void SetUpWaveformPanelInteractions(TSharedRef<STransformedWaveformViewPanel> WaveformPanel, TSharedRef<FSparseSampledSequenceTransportCoordinator> TransportCoordinator, TSharedPtr<FWaveformEditorZoomController> InZoomController = nullptr, TSharedPtr<SWaveformTransformationsOverlay> InTransformationsOverlay = nullptr);
	UE_API FReply HandleTimeRulerInteraction(const TransformedWaveformViewFactory::EReceivedInteractionType MouseInteractionType, TSharedRef<FSparseSampledSequenceTransportCoordinator> TransportCoordinator, const TSharedRef<SWidget> TimeRulerWidget, const FPointerEvent& MouseEvent, const FGeometry& Geometry);
	
	static UE_API TUniquePtr<FTransformedWaveformViewFactory> Instance;

};

#undef UE_API
