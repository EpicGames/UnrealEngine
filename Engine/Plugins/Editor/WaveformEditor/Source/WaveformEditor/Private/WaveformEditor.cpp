// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditor.h"

#include "AssetRegistry/IAssetRegistry.h"
#include "AudioDevice.h"
#include "AudioWidgetsStyle.h"
#include "AssetDefinitionRegistry.h"
#include "Components/AudioComponent.h"
#include "EditorReimportHandler.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Containers/Ticker.h"
#include "Misc/TransactionObjectEvent.h"
#include "PropertyEditorModule.h"
#include "Sound/SoundWave.h"
#include "SparseSampledSequenceTransportCoordinator.h"
#include "STransformedWaveformViewPanel.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "SWaveformTransformationsOverlay.h"
#include "ToolMenus.h"
#include "TransformedWaveformView.h"
#include "TransformedWaveformViewFactory.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "WaveformAudioAnalysisFunctions.h"
#include "WaveformEditorCommands.h"
#include "WaveformEditorDetailsCustomization.h"
#include "WaveformEditorLog.h"
#include "WaveformEditorModule.h"
#include "WaveformEditorSequenceDataProvider.h"
#include "WaveformEditorStyle.h"
#include "WaveformEditorToolMenuContext.h"
#include "WaveformEditorTransformationsSettings.h"
#include "WaveformEditorWaveWriter.h"
#include "WaveformTransformationMarkers.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Notifications/SNotificationList.h"

#define LOCTEXT_NAMESPACE "WaveformEditor"

const FName FWaveformEditor::AppIdentifier("WaveformEditorApp");
const FName FWaveformEditor::PropertiesTabId("WaveformEditor_Properties");
const FName FWaveformEditor::TransformationsTabId("WaveformEditor_Transformations");
const FName FWaveformEditor::WaveformDisplayTabId("WaveformEditor_Display");
const FName FWaveformEditor::EditorName("Waveform Editor");
const FName FWaveformEditor::ToolkitFName("WaveformEditor");
const double FWaveformEditor::EndTimeInvalid = -1.0;

bool FWaveformEditor::Init(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, USoundWave* SoundWaveToEdit)
{
	checkf(SoundWaveToEdit, TEXT("Tried to open a Soundwave Editor from a null soundwave"));

	const TSharedRef<FTabManager::FLayout>  StandaloneDefaultLayout = SetupStandaloneLayout();

	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	const bool bToolbarFocusable = false;
	const bool bUseSmallIcons = true;

	SoundWave = SoundWaveToEdit;

	bool bIsInitialized = false;

	// If either condition is zero it is an indication of a zero length or corrupt file.
	// FWaveformEditorInstantiator::CanSoundWaveBeOpenedInEditor already prevents zero-duration waves from opening through the normal route, added guard here to block in other scenarios (e.g. automated testing)
	if (FMath::IsNearlyZero(SoundWave->GetDuration()) || FMath::IsNearlyZero(SoundWave->TotalSamples))
	{
		UE_LOGF(LogWaveformEditor, Warning, "Waveform Editor attempted to open Zero-Length SoundWave: %ls", *SoundWave->GetName());

		bIsInitialized = false;
	}
	else
	{
		bIsInitialized = true;

		bIsInitialized &= CreateDetailsViews();
		bIsInitialized &= CreateTransportCoordinator();
		bIsInitialized &= InitializeZoom();
		bIsInitialized &= CreateWaveformView();
		bIsInitialized &= InitializeAudioComponent();
		bIsInitialized &= CreateTransportController();
		bIsInitialized &= CreateWaveWriter();
		bIsInitialized &= BindDelegates();
		bIsInitialized &= SetUpAssetReimport();

		bIsInitialized &= RegisterToolbar();
		bIsInitialized &= BindCommands();
	}

	// Must be initialized whether init succeeds or fails for proper destruction
	FAssetEditorToolkit::InitAssetEditor(
		Mode,
		InitToolkitHost,
		AppIdentifier,
		StandaloneDefaultLayout,
		bCreateDefaultStandaloneMenu,
		bCreateDefaultToolbar,
		SoundWaveToEdit,
		bToolbarFocusable,
		bUseSmallIcons);

	if (bIsInitialized)
	{
		if (GEditor)
		{
			GEditor->RegisterForUndo(this);
		}

		// Initialize TransformationChainConfig from persisted transformations or if there are none falling back to default transformations.
		if (SoundWave->Transformations.Num() == 0)
		{
			AddDefaultTransformations();
		}
		else
		{
			TransformationChainConfig = SoundWave->UpdateTransformations();
		}

		// Add a trimfade transformation for user to opt to use, but will be removed from transformations on save if values are zero
		// This does not trigger a modify on the soundwave to avoid marking dirty every time a user opens the waveform, but if they change anything on the trimfade that automatically will modify and mark dirty to save it
		TObjectPtr<UWaveformTransformationTrimFade> TrimFadeTransformation = GetOrAddTrimFadeTransformation(false);
		check(TrimFadeTransformation != nullptr);

		if (FMath::IsNearlyEqual(TrimFadeTransformation->EndTime, EndTimeInvalid))
		{
			TrimFadeTransformation->EndTime = SoundWave->Duration;
		}

		// Initialize waveform view to with playhead at start of active transformations.
		check(WaveformView.DataProvider != nullptr);
		WaveformView.DataProvider->GenerateLayersChain();
		WaveformView.DataProvider->UpdateRenderElements();
		PlaybackTimeBeforeTransformInteraction = 0;
		StartTimeBeforeTransformInteraction = TransformationChainConfig.StartTime; // StartTime=0, EndTime=-1 if no transformations
		check(TransportController != nullptr);
		TransportController->CacheStartTime(PlaybackTimeBeforeTransformInteraction);
		check(TransportCoordinator != nullptr);
		TransportCoordinator->SetProgressRatio(0.0f);

		TWeakPtr<FWaveformEditor> WeakWaveformEditor = SharedThis(this);
		TObjectPtr<UWaveformTransformationMarkers> MarkerTransformation = GetMarkerTransformation();

		if (MarkerTransformation)
		{
			bMarkerTransformationIsActive = true;

			if (!bCueChangeRegisteredByWaveformEditor)
			{
				MarkerTransformation->Markers->CueChanged.BindLambda([WeakWaveformEditor]() 
					{ 
						if (TSharedPtr<FWaveformEditor> PinnedEditor = WeakWaveformEditor.Pin())
						{
							PinnedEditor->RegenerateTransformations();
						}
					});

				bCueChangeRegisteredByWaveformEditor = true;
			}
		}
		else
		{
			bCueChangeRegisteredByWaveformEditor = false;
			bMarkerTransformationIsActive = false;
		}

		BindDelegatesForActiveTransformations();

#if WITH_EDITOR
		if (SoundWave->LUFS == 0.f && SoundWave->SamplePeakDB == 0.f)
		{
			// Update the properties without marking the SoundWave dirty to prevent checkouts unintended by the user
			const bool bWasAlreadyDirty = SoundWave->GetPackage()->IsDirty();

			UpdateSoundWaveProperties();

			if (!bWasAlreadyDirty)
			{
				SoundWave->GetPackage()->ClearDirtyFlag();
			}
		}
#endif // WITH_EDITOR
	
		FSlateApplication::Get().SetKeyboardFocus(TransformationsDetails);
	}

	return bIsInitialized;
}

bool FWaveformEditor::CloseWindow(EAssetEditorCloseReason InCloseReason)
{
	if (InCloseReason == EAssetEditorCloseReason::CloseAllEditorsForAsset
		|| InCloseReason == EAssetEditorCloseReason::AssetUnloadingOrInvalid
		|| InCloseReason == EAssetEditorCloseReason::AssetForceDeleted)
	{
		bSoundWaveBeingDeleted = true;
	}

	return FAssetEditorToolkit::CloseWindow(InCloseReason);
}

FWaveformEditor::~FWaveformEditor()
{
	// Cancel any in-flight deferred render tick — its lambda holds a weak ptr to this editor so it would otherwise still fire and operate on an object that's mid-destruction.
	bDeferredRenderUpdatePending = false;
	FTSTicker::GetCoreTicker().RemoveTicker(DeferredRenderTickerHandle);
	DeferredRenderTickerHandle.Reset();

	if (ChangeCuePointOriginNotificationItem.IsValid())
	{
		ChangeCuePointOriginNotificationItem->Fadeout();
	}

	TObjectPtr<UWaveformTransformationMarkers> MarkerTransformation = GetMarkerTransformation();

	if (MarkerTransformation)
	{
		if (bCueChangeRegisteredByWaveformEditor)
		{
			check(MarkerTransformation->Markers);
			MarkerTransformation->Markers->CueChanged.Unbind();
		}

		// Skip modifying the SoundWave if it is being deleted — modifications are invalid during asset deletion
		if (!bSoundWaveBeingDeleted && TransformationChainConfig.bIsPreviewingLoopRegion)
		{
			// Note: if the user doesn't save the soundwave asset, bLooping will not be reset.
			check(SoundWave);
			SoundWave->Modify();
			SoundWave->bLooping = TransformationChainConfig.bCachedSoundWaveLoopState;

			MarkerTransformation->ResetLoopPreviewing();
			SoundWave->UpdateTransformations();

			NotifySoundWavePostEditChangeProperty(GET_MEMBER_NAME_CHECKED(USoundWave, bLooping));
		}
	}

	TObjectPtr<UWaveformTransformationTrimFade> TrimFadeTransformation = bSoundWaveBeingDeleted ? nullptr : GetTrimFadeTransformation();

	if (TrimFadeTransformation)
	{
		// If the user has not used the trimfade then remove the transformation to avoid unnecessary transformations in the chain
		check(TrimFadeTransformation != nullptr);
		check(TrimFadeTransformation->FadeTransformation != nullptr);
		check(SoundWave != nullptr);

		// Account for a user trimming a single frame by having a 1/2 frame error tolerance
		const uint32 ImportedSampleRate = SoundWave->GetImportedSampleRate();
		check(ImportedSampleRate > 0);

		const double ErrorTolerance = (1.0f / ImportedSampleRate) * 0.5f;

		if (FMath::IsNearlyEqual(TrimFadeTransformation->StartTime, 0, ErrorTolerance) && FMath::IsNearlyEqual(TrimFadeTransformation->EndTime, SoundWave->Duration, ErrorTolerance))
		{
			if (TrimFadeTransformation->FadeTransformation->GetFadeRegionsNum() != 1
				|| (TrimFadeTransformation->GetFadeRegion().FadeIn && FMath::IsNearlyEqual(TrimFadeTransformation->GetFadeRegion().FadeIn->Duration, 0, ErrorTolerance)
					&& TrimFadeTransformation->GetFadeRegion().FadeOut && FMath::IsNearlyEqual(TrimFadeTransformation->GetFadeRegion().FadeOut->Duration, 0, ErrorTolerance))
				|| (TrimFadeTransformation->GetFadeRegion().FadeIn == nullptr && TrimFadeTransformation->GetFadeRegion().FadeOut == nullptr))
			{
				// If the number of Fade Regions is invalid
				// Or if a Fade Region is present and both fades have a zero duration or if both fades are null, remove the transformation
				SoundWave->Transformations.Remove(TrimFadeTransformation);
			}
		}
	}

	UnBindDelegatesForActiveTransformations();

	if (FReimportManager::Instance())
	{
		FReimportManager::Instance()->OnPostReimport().RemoveAll(this);
	}

	if (GEditor)
	{
		GEditor->UnregisterForUndo(this);
	}
}

void FWaveformEditor::AddDefaultTransformations()
{
	if (!ensure(SoundWave))
	{
		return;
	}

	if (SoundWave->Transformations.Num() == 0)
	{
		const UWaveformEditorTransformationsSettings* TransformationsSettings = GetWaveformEditorTransformationsSettings();
		for (const TSubclassOf<UWaveformTransformationBase>& TransformationClass : TransformationsSettings->LaunchTransformations)
		{
			//Adding default transformations does not mark the SoundWave to be saved
			//Modify marks it as dirty to inform the user it needs to be saved if they want the default transformations to remain.
			SoundWave->Modify();
			
			if (TransformationClass)
			{
				if (TransformationClass == UWaveformTransformationTrimFade::StaticClass())
				{
					UE_LOGF(LogWaveformEditor, Warning, "UWaveformTransformationTrimFade is already added automatically and removed if unused. Remove it from LaunchTransformations TSet.");

					continue;
				}

				EObjectFlags MaskedOuterFlags = SoundWave ? SoundWave->GetMaskedFlags(RF_PropagateToSubObjects) : RF_NoFlags;

				if (SoundWave->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
				{
					MaskedOuterFlags |= RF_ArchetypeObject;
				}

				UWaveformTransformationBase* TransformationToAdd = NewObject<UWaveformTransformationBase>(SoundWave, TransformationClass.Get(), NAME_None, MaskedOuterFlags);
				SoundWave->Transformations.Add(TransformationToAdd);
			}
			else
			{
				SoundWave->Transformations.Add(nullptr);
			}

			//Update the content browser asset state to dirty
			NotifySoundWavePostEditChangeProperty(GET_MEMBER_NAME_CHECKED(USoundWave, Transformations), EPropertyChangeType::ArrayAdd);
		}

		TransformationChainConfig = SoundWave->UpdateTransformations();
	}
}

void FWaveformEditor::NotifyPostTransformationChange(const EPropertyChangeType::Type& PropertyChangeType)
{
	check(SoundWave);

	FProperty* TransformationsProperty = FindFProperty<FProperty>(USoundWave::StaticClass(), GET_MEMBER_NAME_CHECKED(USoundWave, Transformations));

	FEditPropertyChain PropertyChain;
	PropertyChain.AddHead(TransformationsProperty);
	PropertyChain.SetActivePropertyNode(TransformationsProperty);

	FPropertyChangedEvent PropertyChangedEvent(TransformationsProperty, PropertyChangeType);

	NotifyPostChange(PropertyChangedEvent, &PropertyChain);

	//Update the content browser asset state to dirty
	NotifySoundWavePostEditChangeProperty(GET_MEMBER_NAME_CHECKED(USoundWave, Transformations), PropertyChangeType);

	TransformationChainConfig = SoundWave->UpdateTransformations();
}

TObjectPtr<UWaveformTransformationTrimFade> FWaveformEditor::GetOrAddTrimFadeTransformation(const bool bMarkModify)
{
	check(SoundWave);

	TObjectPtr<UWaveformTransformationTrimFade> TrimFadeTransformation = GetTrimFadeTransformation();

	if (TrimFadeTransformation == nullptr)
	{
		EObjectFlags MaskedOuterFlags = SoundWave ? SoundWave->GetMaskedFlags(RF_PropagateToSubObjects) : RF_NoFlags;

		if (SoundWave->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
		{
			MaskedOuterFlags |= RF_ArchetypeObject;
		}

		TrimFadeTransformation = NewObject<UWaveformTransformationTrimFade>(SoundWave, UWaveformTransformationTrimFade::StaticClass(), NAME_None, MaskedOuterFlags);

		check(TrimFadeTransformation);
		if (bMarkModify)
		{
			SoundWave->Modify();
			SoundWave->Transformations.Add(TrimFadeTransformation);
			NotifyPostTransformationChange();
		}
		else
		{
			SoundWave->Transformations.Add(TrimFadeTransformation);
		}

		check(TrimFadeTransformation->FadeTransformation);
		check(TrimFadeTransformation->FadeTransformation->GetFadeRegionsNum() == 1);
		// Add default fade curves
		TrimFadeTransformation->GetMutableFadeRegion().FadeIn = NewObject<UTransformationFadeFunction>(TrimFadeTransformation->FadeTransformation.Get(), FadeInType.Get(), NAME_None, RF_Transactional);
		check(TrimFadeTransformation->GetFadeRegion().FadeIn);
		TrimFadeTransformation->GetMutableFadeRegion().FadeOut = NewObject<UTransformationFadeFunction>(TrimFadeTransformation->FadeTransformation.Get(), FadeOutType.Get(), NAME_None, RF_Transactional);
		check(TrimFadeTransformation->GetFadeRegion().FadeOut);

		TrimFadeTransformation->GetFadeRegion().FadeIn->Duration = 0.f;
		TrimFadeTransformation->GetFadeRegion().FadeOut->Duration = 0.f;

		TransformationChainConfig = SoundWave->UpdateTransformations();
	}

	return TrimFadeTransformation;
}

TObjectPtr<UWaveformTransformationTrimFade> FWaveformEditor::GetTrimFadeTransformation() const
{
	if (SoundWave == nullptr)
	{
		return nullptr;
	}

	// Relies on there being only one TrimFade Transformation on the soundwave
	TObjectPtr<UWaveformTransformationBase>* TrimFadeTransformation = SoundWave->Transformations.FindByPredicate([](const TObjectPtr<UWaveformTransformationBase>& Transformation)
		{
			return Transformation.IsA<UWaveformTransformationTrimFade>();
		});

	if (TrimFadeTransformation)
	{
		return CastChecked<UWaveformTransformationTrimFade>(*TrimFadeTransformation);
	}

	return nullptr;
}

TObjectPtr<UWaveformTransformationMarkers> FWaveformEditor::GetOrAddMarkerTransformation()
{
	check(SoundWave);

	TObjectPtr<UWaveformTransformationMarkers> MarkersTransformation = GetMarkerTransformation();

	if (MarkersTransformation == nullptr)
	{
		MarkersTransformation = AddMarkerTransformation();
		PromptToChangeCuePointOrigin();
	}

	return MarkersTransformation;
}

TObjectPtr<UWaveformTransformationMarkers> FWaveformEditor::GetMarkerTransformation() const
{
	if (SoundWave == nullptr)
	{
		return nullptr;
	}

	// Relies on there being only one Markers Transformation on the soundwave
	TObjectPtr<UWaveformTransformationBase>* MarkersTransformation = SoundWave->Transformations.FindByPredicate([](const TObjectPtr<UWaveformTransformationBase>& Transformation)
		{
			return Transformation.IsA<UWaveformTransformationMarkers>();
		});

	if (MarkersTransformation)
	{
		return CastChecked<UWaveformTransformationMarkers>(*MarkersTransformation);
	}

	return nullptr;
}

TObjectPtr<UWaveformTransformationMarkers> FWaveformEditor::AddMarkerTransformation()
{
	TObjectPtr<UWaveformTransformationMarkers> MarkersTransformation;
	EObjectFlags MaskedOuterFlags = SoundWave ? SoundWave->GetMaskedFlags(RF_PropagateToSubObjects) : RF_NoFlags;

	if (SoundWave->HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		MaskedOuterFlags |= RF_ArchetypeObject;
	}

	MarkersTransformation = NewObject<UWaveformTransformationMarkers>(SoundWave, UWaveformTransformationMarkers::StaticClass(), NAME_None, MaskedOuterFlags);

	SoundWave->Modify();
	SoundWave->Transformations.Add(MarkersTransformation);

	NotifyPostTransformationChange();

	return MarkersTransformation;
}

void FWaveformEditor::BindDelegatesForActiveTransformations()
{
	check(SoundWave);
	TWeakPtr<FWaveformEditor> WeakWaveformEditor = SharedThis(this);

	for (const TObjectPtr<UWaveformTransformationBase>& Transformation : SoundWave->Transformations)
	{
		if (Transformation != nullptr)
		{
			if (!Transformation->OnTransformationChanged.IsBound())
			{
				Transformation->OnTransformationChanged.BindLambda([WeakWaveformEditor](bool bModify)
					{
						if (TSharedPtr<FWaveformEditor> PinnedEditor = WeakWaveformEditor.Pin())
						{
							PinnedEditor->UpdateTransformations(bModify);
						}
					});
			}

			if (!Transformation->OnTransformationRenderChanged.IsBound())
			{
				Transformation->OnTransformationRenderChanged.BindLambda([WeakWaveformEditor]()
					{
						if (TSharedPtr<FWaveformEditor> PinnedEditor = WeakWaveformEditor.Pin())
						{
							PinnedEditor->UpdateRenderElements();
						}
					});
			}
		}
	}
}

void FWaveformEditor::UnBindDelegatesForActiveTransformations()
{
	check(SoundWave);
	for (const TObjectPtr<UWaveformTransformationBase>& Transformation : SoundWave->Transformations)
	{
		if (Transformation != nullptr)
		{
			if (Transformation->OnTransformationChanged.IsBound())
			{
				Transformation->OnTransformationChanged.Unbind();
			}
			if (Transformation->OnTransformationRenderChanged.IsBound())
			{
				Transformation->OnTransformationRenderChanged.Unbind();
			}
		}
	}
}

void FWaveformEditor::ToggleFadeIn()
{
	check(SoundWave);
	check(GEditor != nullptr);

	TObjectPtr<UWaveformTransformationTrimFade> TrimFadeTransformation = GetOrAddTrimFadeTransformation();
	check(TrimFadeTransformation);
	check(TrimFadeTransformation->FadeTransformation);
	check(TrimFadeTransformation->FadeTransformation->GetFadeRegionsNum() == 1);

	const FTransformationFadeFunctionData& ConstFadeRegion = TrimFadeTransformation->GetFadeRegion();

	if (ConstFadeRegion.FadeIn)
	{
		GEditor->BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("SetNumericPropertyTransactionToggleFadeIn", "ToggleFadeIn"), nullptr);
		SoundWave->Modify();
		TrimFadeTransformation->Modify();
		ConstFadeRegion.FadeIn->Modify();
		
		if (ConstFadeRegion.FadeIn->Duration == 0.0f)
		{
			check(DefaultFadeInAmount >= 0.0f);

			if (CachedFadeInAmount <= 0.0f)
			{
				CachedFadeInAmount = DefaultFadeInAmount;
			}

			// Ensures the selected subtype (linear, exponential, logarithmic, or sigmoid)
			if (!ConstFadeRegion.FadeIn.IsA(FadeInType.Get()))
			{
				FTransformationFadeFunctionData& MutableFadeRegion = TrimFadeTransformation->GetMutableFadeRegion();
				MutableFadeRegion.FadeIn = NewObject<UTransformationFadeFunction>(TrimFadeTransformation->FadeTransformation.Get(), FadeInType.Get(), NAME_None, RF_Transactional);
				MutableFadeRegion.FadeIn->Modify();

				ensure(&ConstFadeRegion == &MutableFadeRegion);
			}

			ConstFadeRegion.FadeIn->Duration = CachedFadeInAmount;
		}
		else //if fade duration is not 0, assume there is a fade applied and toggle fade "off" by setting fade time to 0
		{
			// No need to check for fade type since we are toggling it "off". Type check is done during toggle "on"
			CachedFadeInAmount = ConstFadeRegion.FadeIn->Duration;

			//No need to set FadeCurve when toggling off. This will preserve user curve data if they later manually change the StartFadeTime.
			ConstFadeRegion.FadeIn->Duration = 0.0f;
		}
		
		NotifyPostTransformationChange(EPropertyChangeType::ValueSet);
		GEditor->EndTransaction();
	}
}

bool FWaveformEditor::CanFadeIn()
{
	return true;
}

void FWaveformEditor::ToggleFadeOut()
{
	check(SoundWave);
	check(GEditor != nullptr);

	TObjectPtr<UWaveformTransformationTrimFade> TrimFadeTransformation = GetOrAddTrimFadeTransformation();
	check(TrimFadeTransformation);
	check(TrimFadeTransformation->FadeTransformation);
	check(TrimFadeTransformation->FadeTransformation->GetFadeRegionsNum() == 1);

	const FTransformationFadeFunctionData& ConstFadeRegion = TrimFadeTransformation->GetFadeRegion();

	if (ConstFadeRegion.FadeOut)
	{
		GEditor->BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("SetNumericPropertyTransactionToggleFadeOut", "ToggleFadeOut"), nullptr);
		SoundWave->Modify();
		TrimFadeTransformation->Modify();
		ConstFadeRegion.FadeOut->Modify();
		
		if (ConstFadeRegion.FadeOut->Duration == 0.0f)
		{
			check(DefaultFadeOutAmount >= 0.0f);

			if (CachedFadeOutAmount <= 0.0f)
			{
				CachedFadeOutAmount = DefaultFadeOutAmount;
			}

			// Ensures the selected subtype (linear, exponential, logarithmic, or sigmoid)
			if (!ConstFadeRegion.FadeOut.IsA(FadeOutType.Get()))
			{
				FTransformationFadeFunctionData& MutableFadeRegion = TrimFadeTransformation->GetMutableFadeRegion();
				MutableFadeRegion.FadeOut = NewObject<UTransformationFadeFunction>(TrimFadeTransformation->FadeTransformation.Get(), FadeOutType.Get(), NAME_None, RF_Transactional);
				MutableFadeRegion.FadeOut->Modify();

				ensure(&ConstFadeRegion == &MutableFadeRegion);
			}

			ConstFadeRegion.FadeOut->Duration = CachedFadeOutAmount;
		}
		else //if fade duration is not 0, assume there is a fade applied and toggle fade "off" by setting fade time to 0
		{
			// No need to check for fade type since we are toggling it "off". Type check is done during toggle "on"
			CachedFadeOutAmount = ConstFadeRegion.FadeOut->Duration;

			//No need to set FadeCurve when toggling off. This will preserve user curve data if they later manually change the StartFadeTime.
			ConstFadeRegion.FadeOut->Duration = 0.0f;
		}

		NotifyPostTransformationChange(EPropertyChangeType::ValueSet);
		GEditor->EndTransaction();
	}
}

bool FWaveformEditor::CanFadeOut()
{
	return true;
}

void FWaveformEditor::CreateMarker(bool bIsLoopRegion)
{
	check(SoundWave);
	check(GEditor != nullptr);
	GEditor->BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("SetNumericPropertyTransactionCreateMarker", "CreateMarker"), nullptr);
	SoundWave->Modify();

	TObjectPtr<UWaveformTransformationMarkers> MarkerTransformation = GetOrAddMarkerTransformation();
	check(MarkerTransformation);

	int32 HighestCueID = INDEX_NONE;

	MarkerTransformation->Markers->Modify();

	for (FSoundWaveCuePoint CuePoint : MarkerTransformation->Markers->CuesAndLoops)
	{
		if (CuePoint.CuePointID > HighestCueID)
		{
			HighestCueID = CuePoint.CuePointID;
		}
	}

	FSoundWaveCuePoint NewCuePoint;
	NewCuePoint.CuePointID = HighestCueID + 1;
	NewCuePoint.FramePosition = WaveformView.ViewWidget->GetPlayheadFramePosition();

	if (bIsLoopRegion)
	{
		NewCuePoint.SetLoopRegion(true);
		NewCuePoint.FrameLength = static_cast<int64>(SoundWave->TotalSamples * 0.1f); // Default loop region to 10% of samples for easier tuning
	}

	MarkerTransformation->Markers->CuesAndLoops.Add(NewCuePoint);
	NotifyPostTransformationChange(EPropertyChangeType::ValueSet);
	GEditor->EndTransaction();
}

void FWaveformEditor::DeleteMarker()
{
	check(SoundWave);
	check(GEditor != nullptr);

	TObjectPtr<UWaveformTransformationMarkers> MarkerTransformation = GetMarkerTransformation();

	if (MarkerTransformation == nullptr)
	{
		return;
	}

	check(MarkerTransformation->Markers);
	
	GEditor->BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("SetNumericPropertyTransactionDeleteMarker", "DeleteMarker"), nullptr);
	SoundWave->Modify();

	int32 CueToDelete = MarkerTransformation->Markers->SelectedCue;

	MarkerTransformation->Markers->Modify();

	if (CueToDelete == INDEX_NONE)
	{
		GEditor->EndTransaction();
		return; // No cue to selected
	}

	for (int i = 0; i < MarkerTransformation->Markers->CuesAndLoops.Num(); i++)
	{
		if (CueToDelete == MarkerTransformation->Markers->CuesAndLoops[i].CuePointID)
		{
			if (MarkerTransformation->Markers->CuesAndLoops[i].IsLoopRegion())
			{
				MarkerTransformation->ResetLoopPreviewing();
			}

			MarkerTransformation->Markers->CuesAndLoops.RemoveAt(i);
			MarkerTransformation->Markers->SelectedCue = INDEX_NONE;

			break;
		}
	}

	NotifyPostTransformationChange(EPropertyChangeType::ValueSet);

	GEditor->EndTransaction();
}

void FWaveformEditor::SkipToNextMarker()
{
	check(SoundWave != nullptr);

	if (TObjectPtr<UWaveformTransformationMarkers> MarkerTransformation = GetOrAddMarkerTransformation())
	{
		check(TransportCoordinator != nullptr);
		check(SoundWave->TotalSamples > 0);
		const int64 PlaybackFramePosition = WaveformView.ViewWidget->GetPlayheadFramePosition();
		int64 NewPlaybackFramePosition = MAX_int64;

		// Find the nearest marker beyond the current playhead.
		check(MarkerTransformation->Markers != nullptr);
		for (int i = 0; i < MarkerTransformation->Markers->CuesAndLoops.Num(); i++)
		{
			if (MarkerTransformation->Markers->CuesAndLoops[i].FramePosition > PlaybackFramePosition + 1)
			{
				if (MarkerTransformation->Markers->CuesAndLoops[i].FramePosition < NewPlaybackFramePosition)
				{
					NewPlaybackFramePosition = MarkerTransformation->Markers->CuesAndLoops[i].FramePosition;
				}
			}
		}

		if (NewPlaybackFramePosition != MAX_int64)
		{
			float ActiveDuration = TransformationChainConfig.EndTime - TransformationChainConfig.StartTime;
			if (ActiveDuration <= 0.0f)
			{
				// If no active+initialized transformation, ActiveDuration will be <= 0.0f, so fallback to sound duration:
				ActiveDuration = SoundWave->Duration; // not GetDuration() to get the raw duration and not INDEFINITELY_LOOPING_DURATION if looping
			}
			if (ActiveDuration > 0)
			{
				WaveformView.ViewWidget->SetPlayheadFramePosition(NewPlaybackFramePosition);
				WaveformView.ViewWidget->ResolveFromFramePosition();
			}
		}
	}
}

void FWaveformEditor::SkipToNextZeroCrossing()
{
	check(SoundWave != nullptr);

	const TArray<float>& PCMArray = WaveformView.DataProvider->GetTransformedPCMData();
	const int64 StartFrame = WaveformView.ViewWidget->GetPlayheadFramePosition();

	float EndTime = TransformationChainConfig.EndTime;
	if (EndTime < 0)
	{
		EndTime = SoundWave->Duration;
	}

	check(TransformationChainConfig.SampleRate > 0);
	const int64 EndFrame = FMath::Max(EndTime * TransformationChainConfig.SampleRate - 1, 0);

	if (EndFrame <= StartFrame || StartFrame < 0)
	{
		return;
	}

	const UWaveformEditorTransformationsSettings* TransformationsSettings = GetWaveformEditorTransformationsSettings();
	check(TransformationsSettings);

	const float NoiseThresholdValue = powf(10.0f, TransformationsSettings->NoiseThreshold / 20.0f);

	check(PCMArray.Num() > 0);
	check(SoundWave->NumChannels > 0);

	const int64 FramesToMove = UWaveformTransformationMarkers::GetFramesToNextZeroCrossing(StartFrame, EndFrame, PCMArray, true, NoiseThresholdValue, SoundWave->NumChannels);

	WaveformView.ViewWidget->SetPlayheadFramePosition(StartFrame + FramesToMove);
	WaveformView.ViewWidget->ResolveFromFramePosition();
}

void FWaveformEditor::RegenerateTransformations()
{
	check(SoundWave);
	WaveformView.DataProvider->GenerateLayersChain();
	WaveformView.DataProvider->UpdateRenderElements();
	TransformationChainConfig = SoundWave->UpdateTransformations();
	
	UpdateTransportState();
}

void FWaveformEditor::UpdateTransformations(bool bMarkDirty)
{
	check(TransportController);
	check(SoundWave);

	bool bIsPlaying = TransportController->IsPlaying();
	bool bWasAlreadyDirty = SoundWave->GetPackage()->IsDirty();
	
	//Update the content browser asset state to dirty
	const EPropertyChangeType::Type ChangeType = bMarkDirty ? EPropertyChangeType::ValueSet : EPropertyChangeType::Interactive;
	NotifySoundWavePostEditChangeProperty(GET_MEMBER_NAME_CHECKED(USoundWave, Transformations), ChangeType);

	TransformationChainConfig = SoundWave->UpdateTransformations();

	FProperty* TransformationsProperty = FindFProperty<FProperty>(USoundWave::StaticClass(), GET_MEMBER_NAME_CHECKED(USoundWave, Transformations));
	check(TransformationsProperty);

	FEditPropertyChain PropertyChain;
	PropertyChain.AddHead(TransformationsProperty);
	PropertyChain.SetActivePropertyNode(TransformationsProperty);

	FPropertyChangedEvent PropertyChangedEvent(TransformationsProperty, ChangeType);

	NotifyPostChange(PropertyChangedEvent, &PropertyChain);

	// Check if we do want to mark the package dirty or not. The user would not expect to modify data when doing things like selecting a loop region
	if (bMarkDirty)
	{
#if WITH_EDITOR
		// Only update SoundWave properties when a change that requires marking it dirty occurs
		UpdateSoundWaveProperties();
#endif //WITH_EDITOR
	}
	else if (!bWasAlreadyDirty)
	{
		SoundWave->GetPackage()->ClearDirtyFlag();
	}

	if (bIsPlaying)
	{
		TransportController->Play();
	}
}

void FWaveformEditor::UpdateRenderElements()
{
	check(SoundWave);

	// During interactive drag, coalesce heavy PCM regens onto a deferred ticker so GenerateSequenceDataInternal doesn't block every mouse-move event.
	// The ticker reads the latest property values when it fires, so visual feedback remains correct.
	if (!bDeferredRenderUpdatePending)
	{
		bDeferredRenderUpdatePending = true;
		TWeakPtr<FWaveformEditor> WeakSelf = SharedThis(this);
		constexpr float DeferredUpdateDelaySeconds = 0.f; // Wait a single tick, then execute

		DeferredRenderTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([WeakSelf](float /*DeltaTime*/) -> bool
			{
				if (TSharedPtr<FWaveformEditor> PinnedSelf = WeakSelf.Pin())
				{
					PinnedSelf->DeferredRenderTickerHandle = FTSTicker::FDelegateHandle();

					// Bail out if the editor is tearing down its SoundWave (asset-delete path) or the view has gone away, the asset-teardown window can fire between schedule and tick.
					if (!PinnedSelf->bDeferredRenderUpdatePending || PinnedSelf->bSoundWaveBeingDeleted || !IsValid(PinnedSelf->SoundWave) || !PinnedSelf->WaveformView.DataProvider.IsValid())
					{
						PinnedSelf->bDeferredRenderUpdatePending = false;
						
						return false;
					}

					PinnedSelf->WaveformView.DataProvider->UpdateRenderElements();
					PinnedSelf->TransformationChainConfig = PinnedSelf->SoundWave->UpdateTransformations();
					PinnedSelf->bDeferredRenderUpdatePending = false;
				}

				return false; // one-shot
			}), DeferredUpdateDelaySeconds);
	}
}

void FWaveformEditor::NotifySoundWavePostEditChangeProperty(const FName& PropertyName, const EPropertyChangeType::Type PropertyChangeType)
{
	check(SoundWave);

	FProperty* Property = USoundWave::StaticClass()->FindPropertyByName(PropertyName);
	check(Property);

	FPropertyChangedEvent PropertyChangedEvent(Property, PropertyChangeType);
	SoundWave->PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangeType != EPropertyChangeType::Interactive)
	{
		SoundWave->MarkPackageDirty();
	}
}

void FWaveformEditor::ModifyMarkerLoopRegion(ELoopModificationControls Modification)
{
	check(SoundWave);
	TObjectPtr<UWaveformTransformationMarkers> MarkersTransformation = GetOrAddMarkerTransformation();
	check(MarkersTransformation);
	check(MarkersTransformation->Markers);

	const TArray<float>& PCMArray = WaveformView.DataProvider->GetTransformedPCMData();
	check(PCMArray.Num() > 0);

	const UWaveformEditorTransformationsSettings* TransformationsSettings = GetWaveformEditorTransformationsSettings();
	check(TransformationsSettings);

	const float NoiseThresholdValue = powf(10.0f, TransformationsSettings->NoiseThreshold / 20.0f);

	MarkersTransformation->ModifyMarkerLoopRegion(Modification, PCMArray, NoiseThresholdValue);
}

void FWaveformEditor::CycleMarkerLoopRegion(ELoopModificationControls Modification)
{
	check(SoundWave);
	TObjectPtr<UWaveformTransformationMarkers> MarkersTransformation = GetOrAddMarkerTransformation();
	check(MarkersTransformation);

	MarkersTransformation->CycleMarkerLoopRegion(Modification);
}

void FWaveformEditor::SetCuePointOrigin(TObjectPtr<UWaveformTransformationMarkers> MarkerTransformation)
{
	if (GEditor)
	{
		if (MarkerTransformation && SoundWave->GetCuePointOrigin() == ESoundWaveCuePointOrigin::WaveFile)
		{
			GEditor->BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("SettingCuePointOrigin", "SetCuePointOrigin"), SoundWave);
			SoundWave->Modify();
			SoundWave->SetCuePointOrigin(ESoundWaveCuePointOrigin::MarkerTransformation);
			NotifySoundWavePostEditChangeProperty(USoundWave::GetCuePointOriginPropertyName());
			GEditor->EndTransaction();
		}
		else if (SoundWave->GetCuePointOrigin() != ESoundWaveCuePointOrigin::WaveFile)
		{
			GEditor->BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("SettingCuePointOrigin", "SetCuePointOrigin"), SoundWave);
			SoundWave->Modify();
			SoundWave->SetCuePointOrigin(ESoundWaveCuePointOrigin::WaveFile);
			NotifySoundWavePostEditChangeProperty(USoundWave::GetCuePointOriginPropertyName());
			GEditor->EndTransaction();
		}
	}
}

void FWaveformEditor::PromptToChangeCuePointOrigin()
{
	if (bMarkerTransformationIsActive)
	{
		return; // Marker transformation already exists, do not prompt user
	}

	TObjectPtr<UWaveformTransformationMarkers> MarkerTransformation = GetMarkerTransformation(); // if marker transformation is valid, it is newly created

	// Checking if the notification item is valid will only allow the user to be prompted once while the waveform editor is open (resets when closed)
	if (MarkerTransformation && SoundWave->GetCuePointOrigin() == ESoundWaveCuePointOrigin::WaveFile && !ChangeCuePointOriginNotificationItem.IsValid())
	{
		bMarkerTransformationIsActive = true;

		FNotificationInfo Info(FText::FromString(TEXT("Do you want to set your soundwave to use cues and loops from your WaveformTransformations?")));
		Info.bFireAndForget = true;
		Info.ExpireDuration = 15.0f;

		Info.ButtonDetails.Add(FNotificationButtonInfo(
			FText::FromString(TEXT("Yes")),
			FText::FromString(TEXT("Set Cue Point Origin to Waveform Transformation Markers")),
			FSimpleDelegate::CreateLambda([this]()
				{
					bool bSuccess = false;
					if (GEditor && SoundWave)
					{
						if (TObjectPtr<UWaveformTransformationMarkers> MarkerTransformation = GetMarkerTransformation())
						{
							SetCuePointOrigin(MarkerTransformation);
							bSuccess = true;
						}
					}
					if (ChangeCuePointOriginNotificationItem.IsValid())
					{
						if (bSuccess)
						{
							ChangeCuePointOriginNotificationItem->SetCompletionState(SNotificationItem::CS_Success);
						}
						else
						{
							ChangeCuePointOriginNotificationItem->SetCompletionState(SNotificationItem::CS_Fail);
						}
						ChangeCuePointOriginNotificationItem->Fadeout();
					}
				})
		));

		Info.ButtonDetails.Add(FNotificationButtonInfo(
			FText::FromString(TEXT("No")),
			FText::FromString(TEXT("Dismiss this message")),
			FSimpleDelegate::CreateLambda([this]()
				{
					if (ChangeCuePointOriginNotificationItem.IsValid())
					{
						ChangeCuePointOriginNotificationItem->SetCompletionState(SNotificationItem::CS_None);
						ChangeCuePointOriginNotificationItem->Fadeout();
					}
				})
		));

		ChangeCuePointOriginNotificationItem = FSlateNotificationManager::Get().AddNotification(Info);

		if (ChangeCuePointOriginNotificationItem.IsValid())
		{
			ChangeCuePointOriginNotificationItem->SetCompletionState(SNotificationItem::CS_Pending);
		}
	}
}

bool FWaveformEditor::InitializeAudioComponent()
{
	if (!ensure(SoundWave))
	{
		return false;
	}

	if (AudioComponent == nullptr)
	{
		if (FAudioDeviceManager* AudioDeviceManager = FAudioDeviceManager::Get())
		{
			if (FAudioDevice* AudioDevice = AudioDeviceManager->GetMainAudioDeviceRaw())
			{
				USoundBase* SoundBase = Cast<USoundBase>(SoundWave);
				AudioComponent = FAudioDevice::CreateComponent(SoundBase);
			}
		}

		if (AudioComponent == nullptr)
		{
			return false;
		}
	}

	AudioComponent->bAutoDestroy = false;
	AudioComponent->bIsUISound = true;
	AudioComponent->bAllowSpatialization = false;
	AudioComponent->bReverb = false;
	AudioComponent->bCenterChannelOnly = false;
	AudioComponent->bIsPreviewSound = true;

	return true;
}

bool FWaveformEditor::CreateTransportController()
{
	if (!ensure(AudioComponent))
	{
		UE_LOGF(LogWaveformEditor, Warning, "Trying to setup transport controls with a null audio component");
		return false;
	}

	TransportController = MakeShared<FWaveformEditorTransportController>(AudioComponent);
	return TransportController != nullptr;
}

bool FWaveformEditor::InitializeZoom()
{
	ZoomManager = MakeShared<FWaveformEditorZoomController>(TransportCoordinator);

	check(TransportCoordinator)
	ZoomManager->OnZoomRatioChanged.AddSP(TransportCoordinator.ToSharedRef(), &FSparseSampledSequenceTransportCoordinator::SetZoomRatio);

	return ZoomManager != nullptr;
}

bool FWaveformEditor::BindDelegates()
{
	if (!ensure(AudioComponent))
	{
		UE_LOGF(LogWaveformEditor, Warning, "Failed to bind to playback percentage change, audio component is null");
		return false;
	}

	AudioComponent->OnAudioPlaybackPercentNative.AddSP(this, &FWaveformEditor::HandlePlaybackPercentageChange);
	AudioComponent->OnAudioPlayStateChangedNative.AddSP(this, &FWaveformEditor::HandleAudioComponentPlayStateChanged);
	TransportCoordinator->OnFocusPointScrubUpdate.AddSP(this, &FWaveformEditor::HandlePlayheadScrub);
	return true;
}

bool FWaveformEditor::SetUpAssetReimport()
{
	if (!FReimportManager::Instance())
	{
		return false;
	}

	FReimportManager::Instance()->OnPostReimport().AddSP(this, &FWaveformEditor::OnAssetReimport);
	return true;
}

void FWaveformEditor::ExecuteReimport()
{
	if (!CanExecuteReimport())
	{
		return;
	}
	
	if (ReimportMode == EWaveEditorReimportMode::SameFileOverwrite)
	{
		ExecuteOverwriteTransformations();
	}

	const bool bSelectNewAsset = ReimportMode == EWaveEditorReimportMode::SelectFile;

	TArray<UObject*> CopyOfSelectedAssets;
	CopyOfSelectedAssets.Add(SoundWave);
	FReimportManager::Instance()->ValidateAllSourceFileAndReimport(CopyOfSelectedAssets, true, -1, bSelectNewAsset);
}

void FWaveformEditor::ExecuteOverwriteTransformations()
{
	for (TObjectPtr<UWaveformTransformationBase> Transformation : SoundWave->Transformations)
	{
		if (Transformation)
		{
			Transformation->OverwriteTransformation();
		}
	}
}

void FWaveformEditor::ResetPlaybackToStart()
{
	check(WaveformView.ViewWidget);
	check(TransportController);
	
	WaveformView.ViewWidget->SetPlayheadFramePosition(TransformationChainConfig.GetStartFrameOffset());
	WaveformView.ViewWidget->ResolveFromFramePosition();
	// Reset the cached start so the next Play() begins at the active region start, not a stale offset. Zero-based within the post-trim region.
	TransportController->CacheStartTime(0.0f);
}

void FWaveformEditor::RegisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	if (TransformationsDetails == nullptr)
	{
		return;
	}

	WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(LOCTEXT("WorkspaceMenu_WaveformEditor", "Sound Wave Editor"));

	FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

	InTabManager->RegisterTabSpawner(PropertiesTabId, FOnSpawnTab::CreateSP(this, &FWaveformEditor::SpawnTab_Properties))
		.SetDisplayName(LOCTEXT("DetailsTab", "Details"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(TransformationsTabId, FOnSpawnTab::CreateSP(this, &FWaveformEditor::SpawnTab_Transformations))
		.SetDisplayName(LOCTEXT("ProcessingTab", "Processing"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

	InTabManager->RegisterTabSpawner(WaveformDisplayTabId, FOnSpawnTab::CreateSP(this, &FWaveformEditor::SpawnTab_WaveformDisplay))
		.SetDisplayName(LOCTEXT("WaveformDisplayTab", "WaveformDisplay"))
		.SetGroup(WorkspaceMenuCategory.ToSharedRef())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Details"));

}

void FWaveformEditor::UnregisterTabSpawners(const TSharedRef<FTabManager>& InTabManager)
{
	InTabManager->UnregisterTabSpawner(PropertiesTabId);
	InTabManager->UnregisterTabSpawner(WaveformDisplayTabId);
}

bool FWaveformEditor::RegisterToolbar()
{
	const FName MenuName = FAssetEditorToolkit::GetToolMenuToolbarName();

	if (!UToolMenus::Get()->IsMenuRegistered(MenuName))
	{
		const FWaveformEditorCommands& Commands = FWaveformEditorCommands::Get();
		UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(MenuName, "AssetEditor.DefaultToolBar", EMultiBoxType::ToolBar);

		if (ToolBar == nullptr)
		{
			return false;
		}

		FToolMenuInsert InsertAfterAssetSection("Asset", EToolMenuInsertType::After);
		FToolMenuSection& PlayBackSection = ToolBar->AddSection("Transport Controls", TAttribute<FText>(), InsertAfterAssetSection);

		FToolMenuEntry PlayEntry = FToolMenuEntry::InitToolBarButton(
			Commands.PlaySoundWave,
			FText(),
			LOCTEXT("WaveformEditorPlayButtonTooltip", "Plays this SoundWave"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.PlayInViewport")
		);

		PlayEntry.StyleNameOverride = FName("Toolbar.BackplateLeftPlay");
		
		FToolMenuEntry PauseEntry = FToolMenuEntry::InitToolBarButton(
			Commands.PauseSoundWave,
			FText(),
			LOCTEXT("WaveformEditorPauseButtonTooltip", "Pauses this SoundWave"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.PausePlaySession.Small")
		);

		PauseEntry.StyleNameOverride = FName("Toolbar.BackplateCenter");

		FToolMenuEntry StopEntry = FToolMenuEntry::InitToolBarButton(
			Commands.StopSoundWave,
			FText(),
			LOCTEXT("WaveformEditorStopButtonTooltip", "Stops this SoundWave"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlayWorld.StopPlaySession.Small")
		);

		StopEntry.StyleNameOverride = FName("Toolbar.BackplateRight");

		PlayBackSection.AddEntry(PlayEntry);
		PlayBackSection.AddEntry(PauseEntry);
		PlayBackSection.AddEntry(StopEntry);


		FToolMenuInsert InsertAfterPlaybackSection("Transport Controls", EToolMenuInsertType::After);
		FToolMenuSection& ZoomSection = ToolBar->AddSection("Zoom Controls", TAttribute<FText>(), InsertAfterPlaybackSection);

		FToolMenuEntry ZoomInEntry = FToolMenuEntry::InitToolBarButton(
			Commands.ZoomIn,
			FText(),
			LOCTEXT("WaveformEditorZoomInButtonTooltip", "Zooms into the soundwave"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Plus")
		);

		FToolMenuEntry ZoomOutEntry = FToolMenuEntry::InitToolBarButton(
			Commands.ZoomOut,
			FText(),
			LOCTEXT("WaveformEditorZoomOutButtonTooltip", "Zooms out the soundwave"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Minus")
		);

		ZoomSection.AddEntry(ZoomInEntry);
		ZoomSection.AddEntry(ZoomOutEntry);

		FToolMenuInsert InsertAfterZoomSection("Zoom Controls", EToolMenuInsertType::After);
		FToolMenuSection& ExportSection = ToolBar->AddSection("Export Controls", TAttribute<FText>(), InsertAfterZoomSection);


		ExportSection.AddDynamicEntry("ExportButton", FNewToolMenuSectionDelegate::CreateLambda([this, Commands](FToolMenuSection& InSection) 
		{
				const UWaveformEditorToolMenuContext* Context = InSection.FindContext<UWaveformEditorToolMenuContext>();

			if (TSharedPtr<FWaveformEditor> LockedObserver = Context->WaveformEditor.Pin())
			{
				FToolMenuEntry ExportEntry = FToolMenuEntry::InitToolBarButton(
					Commands.ExportWaveform,
					FText(),
					TAttribute< FText >::CreateRaw(LockedObserver.Get(), &FWaveformEditor::GetExportButtonToolTip),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.ExportAll")
				);

				InSection.AddEntry(ExportEntry);
			}
		}));

		ExportSection.AddDynamicEntry("ExportOptions", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection) 
		{
				const UWaveformEditorToolMenuContext* Context = InSection.FindContext<UWaveformEditorToolMenuContext>();
				
				if (TSharedPtr<FWaveformEditor> LockedObserver = Context->WaveformEditor.Pin())
				{
					FToolMenuEntry ExportOptionsEntry = FToolMenuEntry::InitComboButton(
						"ExportsOptionsCombo",
						FToolUIActionChoice(FUIAction()),
						FNewToolMenuChoice(FOnGetContent::CreateSP(LockedObserver.Get(), &FWaveformEditor::GenerateExportOptionsMenu)),
						LOCTEXT("ExportsOptions_Label", "Export Options"),
						LOCTEXT("ExportsOptions_ToolTip", "Export options for this waveform"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.ExportAll"),
						true
					);

					InSection.AddEntry(ExportOptionsEntry);
				}				
		}));

		ExportSection.AddDynamicEntry("ImportButton", FNewToolMenuSectionDelegate::CreateLambda([this, Commands](FToolMenuSection& InSection)
		{
			const UWaveformEditorToolMenuContext* Context = InSection.FindContext<UWaveformEditorToolMenuContext>();

			if (TSharedPtr<FWaveformEditor> LockedObserver = Context->WaveformEditor.Pin())
			{
				FToolMenuEntry ReimportEntry = FToolMenuEntry::InitToolBarButton(
					Commands.ReimportAsset,
					FText(),
					TAttribute< FText >::CreateRaw(LockedObserver.Get(), &FWaveformEditor::GetReimportButtonToolTip),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport"));

				InSection.AddEntry(ReimportEntry);
			}
		}));

		ExportSection.AddDynamicEntry("ReimportOptions", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection) 
		{
			const UWaveformEditorToolMenuContext* Context = InSection.FindContext<UWaveformEditorToolMenuContext>();

			if (TSharedPtr<FWaveformEditor> LockedObserver = Context->WaveformEditor.Pin())
			{
				FToolMenuEntry ReimportOptionsEntry = FToolMenuEntry::InitComboButton(
					"ReimportOptionsCombo",
					FToolUIActionChoice(FUIAction()),
					FNewToolMenuChoice(FOnGetContent::CreateSP(LockedObserver.Get(), &FWaveformEditor::GenerateImportOptionsMenu)),
					LOCTEXT("ReimportOptions_Label", "Reimport Options"),
					LOCTEXT("ReimportOptions_ToolTip", "Reimport options for this USoundWave"),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport"),
					true
				);

				InSection.AddEntry(ReimportOptionsEntry);
			}

		}));

		FToolMenuInsert InsertAfterExportSection("Export Controls", EToolMenuInsertType::After);
		FToolMenuSection& TransformationSection = ToolBar->AddSection("Transformation Controls", TAttribute<FText>(), InsertAfterExportSection);

		TransformationSection.AddDynamicEntry("ToggleFadeInButton", FNewToolMenuSectionDelegate::CreateLambda([this, Commands](FToolMenuSection& InSection)
			{
				const UWaveformEditorToolMenuContext* Context = InSection.FindContext<UWaveformEditorToolMenuContext>();

				if (Context && Context->WaveformEditor.IsValid())
				{
					FToolMenuEntry ToggleFadeIn = FToolMenuEntry::InitToolBarButton(
						Commands.ToggleFadeIn,
						FText(),
						LOCTEXT("WaveformEditorFadeInButtonTooltip", "Toggle Fade In Transformation onto the soundwave"),
						FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeIn")
					);
					InSection.AddEntry(ToggleFadeIn);
				}
			}));

		TransformationSection.AddDynamicEntry("FadeInOptions", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
			{
				const UWaveformEditorToolMenuContext* Context = InSection.FindContext<UWaveformEditorToolMenuContext>();

				if (TSharedPtr<FWaveformEditor> LockedObserver = Context->WaveformEditor.Pin())
				{
					FToolMenuEntry ExportOptionsEntry = FToolMenuEntry::InitComboButton(
						"FadeInOptionsCombo",
						FToolUIActionChoice(FUIAction()),
						FNewToolMenuChoice(FOnGetContent::CreateSP(LockedObserver.Get(), &FWaveformEditor::GenerateFadeInOptionsMenu)),
						LOCTEXT("FadeInOptions_Label", "Fade In Options"),
						LOCTEXT("FadeInOptions_ToolTip", "Fade In options for this waveform. Applied when Toggle Fade In is pressed"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.ExportAll"),
						true
					);

					InSection.AddEntry(ExportOptionsEntry);
				}
			}));

		TransformationSection.AddDynamicEntry("ToggleFadeOutButton", FNewToolMenuSectionDelegate::CreateLambda([this, Commands](FToolMenuSection& InSection)
			{
				const UWaveformEditorToolMenuContext* Context = InSection.FindContext<UWaveformEditorToolMenuContext>();

				if (Context && Context->WaveformEditor.IsValid())
				{
					FToolMenuEntry ToggleFadeOut = FToolMenuEntry::InitToolBarButton(
						Commands.ToggleFadeOut,
						FText(),
						LOCTEXT("WaveformEditorFadeOutButtonTooltip", "Toggle Fade Out Transformation onto the soundwave"),
						FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeOut")
					);
					InSection.AddEntry(ToggleFadeOut);
				}
			}));

		TransformationSection.AddDynamicEntry("FadeOutOptions", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
			{
				const UWaveformEditorToolMenuContext* Context = InSection.FindContext<UWaveformEditorToolMenuContext>();

				if (TSharedPtr<FWaveformEditor> LockedObserver = Context->WaveformEditor.Pin())
				{
					FToolMenuEntry ExportOptionsEntry = FToolMenuEntry::InitComboButton(
						"FadeOutOptionsCombo",
						FToolUIActionChoice(FUIAction()),
						FNewToolMenuChoice(FOnGetContent::CreateSP(LockedObserver.Get(), &FWaveformEditor::GenerateFadeOutOptionsMenu)),
						LOCTEXT("FadeOutOptions_Label", "Fade Out Options"),
						LOCTEXT("FadeOutOptions_ToolTip", "Fade Out options for this waveform. Applied when Toggle Fade Out is pressed"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.ExportAll"),
						true
					);

					InSection.AddEntry(ExportOptionsEntry);
				}
			}));

		TransformationSection.AddDynamicEntry("CreateMarkerButton", FNewToolMenuSectionDelegate::CreateLambda([this, Commands](FToolMenuSection& InSection)
			{
				const UWaveformEditorToolMenuContext* Context = InSection.FindContext<UWaveformEditorToolMenuContext>();

				if (Context && Context->WaveformEditor.IsValid())
				{
					FToolMenuEntry CreateMarker = FToolMenuEntry::InitToolBarButton(
						Commands.CreateMarker,
						FText(),
						LOCTEXT("WaveformEditorCreateMarkerButtonTooltip", "Create a marker cue for the soundwave"),
						FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.MarkerAdd")
					);
					InSection.AddEntry(CreateMarker);
				}
			}));


		TransformationSection.AddDynamicEntry("CreateLoopRegionButton", FNewToolMenuSectionDelegate::CreateLambda([this, Commands](FToolMenuSection& InSection)
			{
				const UWaveformEditorToolMenuContext* Context = InSection.FindContext<UWaveformEditorToolMenuContext>();

				if (Context && Context->WaveformEditor.IsValid())
				{
					FToolMenuEntry CreateLoopRegion = FToolMenuEntry::InitToolBarButton(
						Commands.CreateLoopRegion,
						FText(),
						LOCTEXT("WaveformEditorCreateLoopRegionButtonTooltip", "Create a marker loop region for the soundwave"),
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "Sequencer.Transport.Looping")
					);
					InSection.AddEntry(CreateLoopRegion);
				}
			}));

		TransformationSection.AddDynamicEntry("DeleteMarkerButton", FNewToolMenuSectionDelegate::CreateLambda([this, Commands](FToolMenuSection& InSection)
			{
				const UWaveformEditorToolMenuContext* Context = InSection.FindContext<UWaveformEditorToolMenuContext>();

				if (Context && Context->WaveformEditor.IsValid())
				{
					FToolMenuEntry DeleteLoopRegion = FToolMenuEntry::InitToolBarButton(
						Commands.DeleteMarker,
						FText(),
						LOCTEXT("WaveformEditorDeleteMarkerButtonTooltip", "Delete a marker cue or loop region for the soundwave"),
						FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.MarkerDelete")
					);
					InSection.AddEntry(DeleteLoopRegion);
				}
			}));
	}

	return true;
}

TSharedRef<SWidget> FWaveformEditor::GenerateFadeInOptionsMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, GetToolkitCommands());

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("FadeInOptionsSection_Label", "Fade In Options"));
	{
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().FadeInLinear, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeInLinear"));
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().FadeInExponential, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeInExponential"));
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().FadeInLogarithmic, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeInLogarithmic"));
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().FadeInSigmoid, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeInSigmoid"));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FWaveformEditor::GenerateFadeOutOptionsMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, GetToolkitCommands());

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("FadeOutOptionsSection_Label", "Fade Out Options"));
	{
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().FadeOutLinear, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeOutLinear"));
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().FadeOutExponential, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeOutExponential"));
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().FadeOutLogarithmic, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeOutLogarithmic"));
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().FadeOutSigmoid, NAME_None, TAttribute<FText>(), TAttribute<FText>(), FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeOutSigmoid"));
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FWaveformEditor::GenerateExportOptionsMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, GetToolkitCommands());
	
	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ChannelSection_Label", "Export Channel Format"));
	{
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().ExportFormatMono);
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().ExportFormatStereo);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FWaveformEditor::GenerateImportOptionsMenu()
{
	const bool bShouldCloseWindowAfterMenuSelection = true;
	FMenuBuilder MenuBuilder(bShouldCloseWindowAfterMenuSelection, GetToolkitCommands());

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("ReimportMode_Label", "Reimport Mode"));
	{
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().ReimportModeSameFile);
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().ReimportModeSameFileOverwriteTransformations);
		MenuBuilder.AddMenuEntry(FWaveformEditorCommands::Get().ReimportModeNewFile);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

bool FWaveformEditor::CanExecuteReimport() const
{
	if (FReimportManager::Instance() == nullptr)
	{
		return false;
	}

	return FReimportManager::Instance()->CanReimport(SoundWave);
}

bool FWaveformEditor::BindCommands()
{
	check(TransportController.IsValid());
	check(ZoomManager.IsValid());
	check(WaveWriter.IsValid());

	const FWaveformEditorCommands& Commands = FWaveformEditorCommands::Get();

	ToolkitCommands->MapAction(
		Commands.PlaySoundWave,
		FExecuteAction::CreateSP(TransportController.ToSharedRef(), &FWaveformEditorTransportController::Play),
		FCanExecuteAction::CreateSP(this, &FWaveformEditor::CanPressPlayButton));

	ToolkitCommands->MapAction(
		Commands.StopSoundWave,
		FExecuteAction::CreateSP(TransportController.ToSharedRef(), &FWaveformEditorTransportController::Stop),
		FCanExecuteAction::CreateSP(TransportController.ToSharedRef(), &FWaveformEditorTransportController::CanStop));

	ToolkitCommands->MapAction(
		Commands.TogglePlayback,
		FExecuteAction::CreateSP(TransportController.ToSharedRef(), &FWaveformEditorTransportController::TogglePlayback));

	ToolkitCommands->MapAction(
		Commands.PauseSoundWave,
		FExecuteAction::CreateSP(TransportController.ToSharedRef(), &FWaveformEditorTransportController::Pause),
		FCanExecuteAction::CreateSP(TransportController.ToSharedRef(), &FWaveformEditorTransportController::IsPlaying));

	ToolkitCommands->MapAction(
		Commands.ReturnToStart,
		FExecuteAction::CreateSP(this, &FWaveformEditor::ResetPlaybackToStart));

	ToolkitCommands->MapAction(
		Commands.ZoomIn,
		FExecuteAction::CreateSP(ZoomManager.ToSharedRef(), &FWaveformEditorZoomController::ZoomIn),
		FCanExecuteAction::CreateSP(ZoomManager.ToSharedRef(), &FWaveformEditorZoomController::CanZoomIn));

	ToolkitCommands->MapAction(
		Commands.ZoomOut,
		FExecuteAction::CreateSP(ZoomManager.ToSharedRef(), &FWaveformEditorZoomController::ZoomOut),
		FCanExecuteAction::CreateSP(ZoomManager.ToSharedRef(), &FWaveformEditorZoomController::CanZoomOut));

	constexpr bool bIsAsync = true;
	
	ToolkitCommands->MapAction(
		Commands.ExportWaveform,
		FExecuteAction::CreateSP(this, &FWaveformEditor::ExportWaveform, bIsAsync),
		FCanExecuteAction::CreateSP(WaveWriter.ToSharedRef(), &FWaveformEditorWaveWriter::CanCreateSoundWaveAsset));

	ToolkitCommands->MapAction(
		Commands.ExportFormatMono,
		FExecuteAction::CreateLambda([this] { (WaveWriter->SetExportChannelsFormat(WaveformEditorWaveWriter::EChannelFormat::Mono)); }),
		FCanExecuteAction::CreateLambda([this] {return WaveWriter.IsValid(); }),
		FIsActionChecked::CreateLambda([this] {return WaveWriter->GetExportChannelsFormat() == WaveformEditorWaveWriter::EChannelFormat::Mono; }));

	ToolkitCommands->MapAction(
		Commands.ExportFormatStereo,
		FExecuteAction::CreateLambda([this] { (WaveWriter->SetExportChannelsFormat(WaveformEditorWaveWriter::EChannelFormat::Stereo)); }),
		FCanExecuteAction::CreateLambda([this] {return WaveWriter.IsValid(); }),
		FIsActionChecked::CreateLambda([this] {return WaveWriter->GetExportChannelsFormat() == WaveformEditorWaveWriter::EChannelFormat::Stereo; }));

	ToolkitCommands->MapAction(
		Commands.ReimportAsset,
		FExecuteAction::CreateSP(this, &FWaveformEditor::ExecuteReimport),
		FCanExecuteAction::CreateSP(this, &FWaveformEditor::CanExecuteReimport));

	ToolkitCommands->MapAction(
		Commands.ReimportModeSameFile,
		FExecuteAction::CreateLambda([this] { ReimportMode = EWaveEditorReimportMode::SameFile; }),
		FCanExecuteAction::CreateLambda([this] { return CanExecuteReimport(); }),
		FIsActionChecked::CreateLambda([this] {return ReimportMode == EWaveEditorReimportMode::SameFile; }));

	ToolkitCommands->MapAction(
		Commands.ReimportModeSameFileOverwriteTransformations,
		FExecuteAction::CreateLambda([this] { ReimportMode = EWaveEditorReimportMode::SameFileOverwrite; }),
		FCanExecuteAction::CreateLambda([this] { return CanExecuteReimport(); }),
		FIsActionChecked::CreateLambda([this] {return ReimportMode == EWaveEditorReimportMode::SameFileOverwrite; }));

	ToolkitCommands->MapAction(
		Commands.ReimportModeNewFile,
		FExecuteAction::CreateLambda([this] { ReimportMode = EWaveEditorReimportMode::SelectFile; }),
		FCanExecuteAction::CreateLambda([this] { return CanExecuteReimport(); }),
		FIsActionChecked::CreateLambda([this] { return ReimportMode == EWaveEditorReimportMode::SelectFile; }));

	ToolkitCommands->MapAction(
		Commands.ToggleFadeIn,
		FExecuteAction::CreateSP(this, &FWaveformEditor::ToggleFadeIn),
		FCanExecuteAction::CreateSP(this, &FWaveformEditor::CanFadeIn));

	ToolkitCommands->MapAction(
		Commands.FadeInLinear,
		FExecuteAction::CreateLambda([this] 
			{ 
				FadeInType = UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Linear];
			}),
		FCanExecuteAction::CreateLambda([this] { return true; }),
		FIsActionChecked::CreateLambda([this] { return FadeInType == UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Linear]; }));

	ToolkitCommands->MapAction(
		Commands.FadeInExponential,
		FExecuteAction::CreateLambda([this] 
			{ 
				FadeInType = UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Exponential];
			}),
		FCanExecuteAction::CreateLambda([this] { return true; }),
		FIsActionChecked::CreateLambda([this] { return FadeInType == UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Exponential]; }));

	ToolkitCommands->MapAction(
		Commands.FadeInLogarithmic,
		FExecuteAction::CreateLambda([this] 
			{ 
				FadeInType = UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Logarithmic];
			}),
		FCanExecuteAction::CreateLambda([this] { return true; }),
		FIsActionChecked::CreateLambda([this] { return FadeInType == UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Logarithmic]; }));

	ToolkitCommands->MapAction(
		Commands.FadeInSigmoid,
		FExecuteAction::CreateLambda([this] 
			{ 
				FadeInType = UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Sigmoid];
			}),
		FCanExecuteAction::CreateLambda([this] { return true; }),
		FIsActionChecked::CreateLambda([this] { return FadeInType == UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Sigmoid]; }));

	ToolkitCommands->MapAction(
		Commands.ToggleFadeOut,
		FExecuteAction::CreateSP(this, &FWaveformEditor::ToggleFadeOut),
		FCanExecuteAction::CreateSP(this, &FWaveformEditor::CanFadeOut));

	ToolkitCommands->MapAction(
		Commands.FadeOutLinear,
		FExecuteAction::CreateLambda([this] 
			{ 
				FadeOutType = UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Linear];
			}),
		FCanExecuteAction::CreateLambda([this] {return true; }),
		FIsActionChecked::CreateLambda([this] {return FadeOutType == UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Linear]; }));

	ToolkitCommands->MapAction(
		Commands.FadeOutExponential,
		FExecuteAction::CreateLambda([this] 
			{ 
				FadeOutType = UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Exponential];
			}),
		FCanExecuteAction::CreateLambda([this] {return true; }),
		FIsActionChecked::CreateLambda([this] {return FadeOutType == UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Exponential]; }));

	ToolkitCommands->MapAction(
		Commands.FadeOutLogarithmic,
		FExecuteAction::CreateLambda([this] 
			{ 
				FadeOutType = UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Logarithmic];
			}),
		FCanExecuteAction::CreateLambda([this] {return true; }),
		FIsActionChecked::CreateLambda([this] {return FadeOutType == UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Logarithmic]; }));

	ToolkitCommands->MapAction(
		Commands.FadeOutSigmoid,
		FExecuteAction::CreateLambda([this] 
			{ 
				FadeOutType = UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Sigmoid];
			}),
		FCanExecuteAction::CreateLambda([this] {return true; }),
		FIsActionChecked::CreateLambda([this] {return FadeOutType == UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Sigmoid]; }));

	ToolkitCommands->MapAction(
		Commands.LeftBoundsIncrease,
		FExecuteAction::CreateLambda([this] { ModifyMarkerLoopRegion(ELoopModificationControls::LeftHandleIncrement); }));

	ToolkitCommands->MapAction(
		Commands.LeftBoundsDecrease,
		FExecuteAction::CreateLambda([this] { ModifyMarkerLoopRegion(ELoopModificationControls::LeftHandleDecrement); }));

	ToolkitCommands->MapAction(
		Commands.RightBoundsIncrease,
		FExecuteAction::CreateLambda([this] { ModifyMarkerLoopRegion(ELoopModificationControls::RightHandleIncrement); }));

	ToolkitCommands->MapAction(
		Commands.RightBoundsDecrease,
		FExecuteAction::CreateLambda([this] { ModifyMarkerLoopRegion(ELoopModificationControls::RightHandleDecrement); }));

	ToolkitCommands->MapAction(
		Commands.LeftBoundsNextZeroCrossing,
		FExecuteAction::CreateLambda([this] { ModifyMarkerLoopRegion(ELoopModificationControls::LeftHandleNextZeroCrossing); }));

	ToolkitCommands->MapAction(
		Commands.LeftBoundsPreviousZeroCrossing,
		FExecuteAction::CreateLambda([this] { ModifyMarkerLoopRegion(ELoopModificationControls::LeftHandlePreviousZeroCrossing); }));

	ToolkitCommands->MapAction(
		Commands.RightBoundsNextZeroCrossing,
		FExecuteAction::CreateLambda([this] { ModifyMarkerLoopRegion(ELoopModificationControls::RightHandleNextZeroCrossing); }));

	ToolkitCommands->MapAction(
		Commands.RightBoundsPreviousZeroCrossing,
		FExecuteAction::CreateLambda([this] { ModifyMarkerLoopRegion(ELoopModificationControls::RightHandlePreviousZeroCrossing); }));

	ToolkitCommands->MapAction(
		Commands.BoundsIncrementIncrease,
		FExecuteAction::CreateLambda([this] { ModifyMarkerLoopRegion(ELoopModificationControls::IncreaseIncrement); }));

	ToolkitCommands->MapAction(
		Commands.BoundsIncrementDecrease,
		FExecuteAction::CreateLambda([this] { ModifyMarkerLoopRegion(ELoopModificationControls::DecreaseIncrement); }));

	ToolkitCommands->MapAction(
		Commands.SelectNextLoop,
		FExecuteAction::CreateLambda([this] { CycleMarkerLoopRegion(ELoopModificationControls::SelectNextLoop); }));

	ToolkitCommands->MapAction(
		Commands.SelectPreviousLoop,
		FExecuteAction::CreateLambda([this] { CycleMarkerLoopRegion(ELoopModificationControls::SelectPreviousLoop); }));

	ToolkitCommands->MapAction(
		Commands.CreateMarker,
		FExecuteAction::CreateLambda([this] { FWaveformEditor::CreateMarker(false); }));

	ToolkitCommands->MapAction(
		Commands.CreateLoopRegion,
		FExecuteAction::CreateLambda([this] { FWaveformEditor::CreateMarker(true); }));

	ToolkitCommands->MapAction(
		Commands.DeleteMarker,
		FExecuteAction::CreateSP(this, &FWaveformEditor::DeleteMarker));

	ToolkitCommands->MapAction(
		Commands.SkipToNextMarker,
		FExecuteAction::CreateSP(this, &FWaveformEditor::SkipToNextMarker));

	ToolkitCommands->MapAction(
		Commands.SkipToNextZeroCrossing,
		FExecuteAction::CreateSP(this, &FWaveformEditor::SkipToNextZeroCrossing));

	return true;
}


FName FWaveformEditor::GetEditorName() const
{
	return EditorName;
}

FName FWaveformEditor::GetToolkitFName() const
{
	return ToolkitFName;
}

FText FWaveformEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Waveform Editor");
}

EVisibility FWaveformEditor::GetVisibilityWhileAssetCompiling() const
{
	return EVisibility::Visible;
}

FString FWaveformEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Waveform Editor").ToString();
}

FLinearColor FWaveformEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor(0.0f, 0.0f, 0.2f, 0.5f);
}

void FWaveformEditor::OnAssetReimport(UObject* ReimportedObject, bool bSuccessfullReimport)
{
	if (!bSuccessfullReimport)
	{
		return;
	}

	if (ReimportedObject->IsA<USoundWave>() && ReimportedObject->GetPathName() == SoundWave->GetPathName())
	{
		check(SoundWave);

		//If the waveform editor is open, updating Transformations on reimport will show
		// the effects of the overwritten transformations immediately instead of after a
		// change is made by the user.
		TransformationChainConfig = SoundWave->UpdateTransformations();
		CreateWaveformView();

		check(WaveformView.DataProvider);
		check(WaveformView.ViewWidget);
		WaveformView.DataProvider->RequestSequenceView(TransportCoordinator->GetDisplayRange());
		WaveformView.ViewWidget->SetPlayheadRatio(TransportCoordinator->GetFocusPoint());

		if (TSharedPtr<SDockTab> LiveTab = this->TabManager->FindExistingLiveTab(WaveformDisplayTabId))
		{
			LiveTab->SetContent(WaveformView.ViewWidget.ToSharedRef());
		}
	}
}


void FWaveformEditor::NotifyPreChange(FEditPropertyChain* PropertyAboutToChange)
{
	check(TransportController != nullptr);
	bWasPlayingBeforeChange = TransportController->IsPlaying();
}

void FWaveformEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged)
{
	TDoubleLinkedList<FProperty*>::TDoubleLinkedListNode* PropertyNode = PropertyThatChanged->GetActiveMemberNode();

	if (!PropertyNode)
	{
		return;
	}

	bool bIsTransformation = false;

	do
	{
		bIsTransformation |= PropertyNode->GetValue()->GetName() == TEXT("Transformations");
		PropertyNode = PropertyNode->GetPrevNode();
	} while (PropertyNode != nullptr);

	if (!bIsTransformation)
	{
		return;
	}

	const bool bIsInteractive = (PropertyChangedEvent.ChangeType == EPropertyChangeType::Interactive);

	if (bIsInteractive)
	{
		bIsInteractingWithTransformations = true;

		if (TransportController->IsPlaying() || TransportController->IsPaused())
		{
			TransformInteractionPlayState = AudioComponent->GetPlayState();
			PlaybackTimeBeforeTransformInteraction = LastReceivedPlaybackPercent * (TransformationChainConfig.EndTime - TransformationChainConfig.StartTime);
			StartTimeBeforeTransformInteraction = TransformationChainConfig.StartTime;
			AudioComponent->Stop();
			bWasPlayingBeforeChange = false;
		}
	}

	const bool bUpdateTransformationChain = PropertyChangedEvent.GetPropertyName() == TEXT("Transformations");
	if (bUpdateTransformationChain)
	{
		WaveformView.DataProvider->GenerateLayersChain();
	}

	// Non-interactive changes (mouse-release ValueSet, programmatic edits) run immediately and supersede any pending deferred update.
	if (bIsInteractive)
	{
		UpdateRenderElements();
	}
	else
	{
		// Immediate update supersedes any pending deferred regen, cancel the in-flight ticker so it doesn't fire a stale no-op next frame.
		bDeferredRenderUpdatePending = false;
		FTSTicker::GetCoreTicker().RemoveTicker(DeferredRenderTickerHandle);
		DeferredRenderTickerHandle = FTSTicker::FDelegateHandle();
		WaveformView.DataProvider->UpdateRenderElements();
	}
	
	TransformationChainConfig = SoundWave->GetTransformationChainConfig();

	TWeakPtr<FWaveformEditor> WeakWaveformEditor = SharedThis(this);
	TObjectPtr<UWaveformTransformationMarkers> MarkerTransformation = GetMarkerTransformation();

	if (MarkerTransformation != nullptr)
	{
		if (!bCueChangeRegisteredByWaveformEditor)
		{
			MarkerTransformation->Markers->CueChanged.BindLambda([WeakWaveformEditor]()
				{
					if (TSharedPtr<FWaveformEditor> PinnedEditor = WeakWaveformEditor.Pin())
					{
						PinnedEditor->RegenerateTransformations();
					}
				});

			bCueChangeRegisteredByWaveformEditor = true;
		}
	}
	else
	{
		bCueChangeRegisteredByWaveformEditor = false;
		bMarkerTransformationIsActive = false;
	}

	BindDelegatesForActiveTransformations();

	UpdateTransportState();

	bIsInteractingWithTransformations = false;

	PromptToChangeCuePointOrigin();
}

void FWaveformEditor::UpdateTransportState()
{
	if (!TransportController->IsPlaying())
	{
		// StartTime=0, EndTime=-1 if no transformations
		const float TransformationDuration = TransformationChainConfig.EndTime - TransformationChainConfig.StartTime;

		// Clamp Playback/Start times to start if out of range of TransformationChainConfig.
		// If no active+initialized transformation, TransformationDuration will be <= 0.0f, ignore in this case.
		if ((PlaybackTimeBeforeTransformInteraction + StartTimeBeforeTransformInteraction - TransformationChainConfig.StartTime <= 0)
			|| (TransformationDuration > 0.0f && PlaybackTimeBeforeTransformInteraction + StartTimeBeforeTransformInteraction - TransformationChainConfig.StartTime >= TransformationDuration))
		{
			PlaybackTimeBeforeTransformInteraction = 0;
			StartTimeBeforeTransformInteraction = TransformationChainConfig.StartTime;
		}
		const float StartTimeDifference = StartTimeBeforeTransformInteraction - TransformationChainConfig.StartTime;
		const float AdjustedPlaybackTime = PlaybackTimeBeforeTransformInteraction + StartTimeDifference;

		switch (TransformInteractionPlayState)
		{
		case EAudioComponentPlayState::Playing:
			TransportController->Play(AdjustedPlaybackTime);
			TransformInteractionPlayState = EAudioComponentPlayState::Stopped;
			break;
		case EAudioComponentPlayState::Paused:
			TransformInteractionPlayState = EAudioComponentPlayState::Stopped;
		case EAudioComponentPlayState::Stopped:
			// Set TransportController and displayed TransportCoordinator to value clamped within TransformationChainConfig.

			if (!TransportCoordinator->IsScrubbing() && !TransportCoordinator->IsScrolling())
			{
				check(SoundWave->GetImportedSampleRate() > 0);
				const float PlayheadTime = static_cast<float>(WaveformView.ViewWidget->GetPlayheadFramePosition()) / static_cast<float>(SoundWave->GetImportedSampleRate());

				// If the playhead goes out of bounds, snap it to the closest trim edge
				if (PlayheadTime > TransformationChainConfig.EndTime)
				{
					TransportController->CacheStartTime(TransformationChainConfig.EndTime - TransformationChainConfig.StartTime);
					TransportCoordinator->SetProgressRatio(1.0f);

					// Cache a play time at the end if the end time trim is what caused the playhead to move
					StartTimeBeforeTransformInteraction = TransformationChainConfig.EndTime;
				}
				else if (PlayheadTime < TransformationChainConfig.StartTime)
				{
					const float CachedAudioStartTimeAsPercentage = TransportController->GetCachedAudioStartTimeAsPercentage();
					TransportCoordinator->SetProgressRatio(CachedAudioStartTimeAsPercentage);
				}
				else
				{
					TransportController->CacheStartTime(AdjustedPlaybackTime);
				}
			}
			else
			{
				TransportController->CacheStartTime(AdjustedPlaybackTime);
			}
			break;
		default:
			break;
		}

		if (bWasPlayingBeforeChange)
		{
			AudioComponent->Play();
		}
	}
}

#if WITH_EDITOR
void FWaveformEditor::UpdateSoundWaveProperties()
{
	check(SoundWave);
	check(WaveformView.DataProvider);

	const TArray<float>& TransformedPCMData = WaveformView.DataProvider->GetTransformedPCMData();
	const uint32 SampleRate = TransformationChainConfig.SampleRate != 0 ?
		TransformationChainConfig.SampleRate : SoundWave->GetImportedSampleRate();
	
	if (TransformedPCMData.IsEmpty() || SampleRate == 0)
	{		
		return;
	}
	
	const int64 StartSample = FMath::RoundToInt64(TransformationChainConfig.StartTime * SampleRate * TransformationChainConfig.NumChannels);
	const int64 EndSample = FMath::RoundToInt64(TransformationChainConfig.EndTime * SampleRate * TransformationChainConfig.NumChannels);

	const Audio::FAlignedFloatBuffer AudioBuffer(TransformedPCMData.GetData() + StartSample, FMath::Clamp(EndSample - StartSample, 0, TransformedPCMData.Num() - StartSample));

	const float SamplePeakDB = WaveformAudioAnalysis::GetPeakSampleValue(AudioBuffer);
	const float LUFS = WaveformAudioAnalysis::GetLUFS(AudioBuffer, SampleRate, TransformationChainConfig.NumChannels);

	SoundWave->SetLoudnessValues(LUFS, SamplePeakDB);

	// Refresh the asset registry tags so the content browser tooltip displays updated metadata.
	// This avoids PostEditChangeProperty which would trigger an unnecessary derived data rebuild.
	IAssetRegistry::GetChecked().AssetUpdateTags(SoundWave, EAssetRegistryTagsCaller::FullUpdate);
}
#endif //WITH_EDITOR

void FWaveformEditor::PostUndo(bool bSuccess)
{
	if (bSuccess)
	{
		WaveformView.DataProvider->GenerateLayersChain();
		WaveformView.DataProvider->UpdateRenderElements();

		TObjectPtr<UWaveformTransformationMarkers> MarkerTransformation = GetMarkerTransformation();

		if (MarkerTransformation != nullptr && !bCueChangeRegisteredByWaveformEditor)
		{
			MarkerTransformation->Markers->CueChanged.BindLambda([this]() { RegenerateTransformations(); });
			bCueChangeRegisteredByWaveformEditor = true;
		}
		else if (MarkerTransformation == nullptr)
		{
			bCueChangeRegisteredByWaveformEditor = false;
		}
	}
}

void FWaveformEditor::PostRedo(bool bSuccess)
{
	PostUndo(bSuccess);
}

bool FWaveformEditor::MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const
{
	bool bShouldUndo = false;
	for (const TPair<UObject*, FTransactionObjectEvent>& TransactionObjectPair : TransactionObjectContexts)
	{
		if (const UObject* Object = TransactionObjectPair.Key)
		{
			if (const UClass* ObjectClass = Object->GetClass())
			{
				bShouldUndo |= ObjectClass->IsChildOf(UWaveformTransformationBase::StaticClass()) || ObjectClass->IsChildOf(USoundWave::StaticClass());
			}
		}
	}

	return bShouldUndo;
}

void FWaveformEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	FAssetEditorToolkit::InitToolMenuContext(MenuContext);

	UWaveformEditorToolMenuContext* Context = NewObject<UWaveformEditorToolMenuContext>();
	Context->WaveformEditor = SharedThis(this);
	MenuContext.AddObject(Context);
}

void FWaveformEditor::ExportWaveform(bool bIsAsync)
{
	check(WaveWriter);
	WaveWriter->ExportTransformedWaveform(bIsAsync);
}

bool FWaveformEditor::CreateDetailsViews()
{
	if (!ensure(SoundWave)) 
	{
		UE_LOGF(LogWaveformEditor, Warning, "Trying to setup wav editor properties view from a null SoundWave");
		return false;
	}

	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
	FDetailsViewArgs Args;
	Args.bHideSelectionTip = true;
	Args.NotifyHook = this;

	PropertiesDetails = PropertyModule.CreateDetailView(Args);
	PropertiesDetails->SetObject(SoundWave);

	TransformationsDetails = PropertyModule.CreateDetailView(Args);
	FOnGetDetailCustomizationInstance TransformationsDetailsCustomizationInstance = FOnGetDetailCustomizationInstance::CreateLambda([]() {
			return MakeShared<FWaveformTransformationsDetailsCustomization>();
		}
	);

	TransformationsDetails->RegisterInstancedCustomPropertyLayout(SoundWave->GetClass(), TransformationsDetailsCustomizationInstance);
	TransformationsDetails->SetObject(SoundWave);

	return true;
}

TSharedRef<SDockTab> FWaveformEditor::SpawnTab_WaveformDisplay(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == WaveformDisplayTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("WaveformDisplayTitle", "Waveform Display"))
		[
			WaveformView.ViewWidget.ToSharedRef()
		];
}

const TSharedRef<FTabManager::FLayout> FWaveformEditor::SetupStandaloneLayout()
{
	const TSharedRef<FTabManager::FLayout> StandaloneDefaultLayout = FTabManager::NewLayout("Standalone_WaveformEditor_v2")
		->AddArea
		(
			FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
			->Split
			(
				FTabManager::NewSplitter()->SetOrientation(Orient_Horizontal)->SetSizeCoefficient(0.9f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.2f)
					->AddTab(PropertiesTabId, ETabState::OpenedTab)
					->AddTab(TransformationsTabId, ETabState::OpenedTab)
					->SetForegroundTab(PropertiesTabId)

				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.8f)
					->SetHideTabWell(true)
					->AddTab(WaveformDisplayTabId, ETabState::OpenedTab)
				)
			)
		);

	return StandaloneDefaultLayout;
}

TSharedRef<SDockTab> FWaveformEditor::SpawnTab_Properties(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == PropertiesTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("SoundWaveDetailsTitle", "Details"))
		[
			PropertiesDetails.ToSharedRef()
		];
}

TSharedRef<SDockTab> FWaveformEditor::SpawnTab_Transformations(const FSpawnTabArgs& Args)
{
	check(Args.GetTabId() == TransformationsTabId);

	return SNew(SDockTab)
		.Label(LOCTEXT("SoundWaveProcessingTitle", "Processing"))
		[
			TransformationsDetails.ToSharedRef()
		];
}

bool FWaveformEditor::CreateWaveformView()
{
	if (!ensure(SoundWave))
	{
		UE_LOGF(LogWaveformEditor, Warning, "Trying to setup waveform panel from a null SoundWave");
		return false;
	}
	
	if (WaveformView.IsValid())
	{
		RemoveWaveformViewDelegates(*WaveformView.DataProvider, *WaveformView.ViewWidget);
	}

	WaveformView = FTransformedWaveformViewFactory::Get().GetTransformedView(SoundWave, TransportCoordinator.ToSharedRef(), this, ZoomManager);

	check(ZoomManager)

	BindWaveformViewDelegates(*WaveformView.DataProvider, *WaveformView.ViewWidget);

	ZoomManager->OnZoomRatioChanged.AddSP(TransportCoordinator.ToSharedRef(), &FSparseSampledSequenceTransportCoordinator::SetZoomRatio);
	TransportCoordinator->OnDisplayRangeUpdated.AddSP(this, &FWaveformEditor::HandleDisplayRangeUpdate);

	return WaveformView.IsValid();
}

void FWaveformEditor::BindWaveformViewDelegates(FWaveformEditorSequenceDataProvider& ViewDataProvider, STransformedWaveformViewPanel& ViewWidget)
{
	check(TransportCoordinator)

	ViewDataProvider.OnRenderElementsUpdated.AddSP(this, &FWaveformEditor::HandleRenderDataUpdate);
	TransportCoordinator->OnFocusPointMoved.AddSP(&ViewWidget, &STransformedWaveformViewPanel::SetPlayheadRatio);
}

void FWaveformEditor::RemoveWaveformViewDelegates(FWaveformEditorSequenceDataProvider& ViewDataProvider, STransformedWaveformViewPanel& ViewWidget)
{
	check(TransportCoordinator)

	ViewDataProvider.OnRenderElementsUpdated.RemoveAll(this);
	TransportCoordinator->OnFocusPointMoved.RemoveAll(&ViewWidget);
}

bool FWaveformEditor::CreateTransportCoordinator()
{
	TransportCoordinator = MakeShared<FSparseSampledSequenceTransportCoordinator>();
	TransportCoordinator->OnDisplayRangeUpdated.AddSP(this, &FWaveformEditor::HandleDisplayRangeUpdate);

	return TransportCoordinator != nullptr;
}

void FWaveformEditor::HandlePlaybackPercentageChange(const UAudioComponent* InComponent, const USoundWave* InSoundWave, const float InPlaybackPercentage)
{
	const bool bIsStopped = AudioComponent->GetPlayState() == EAudioComponentPlayState::Stopped;
	const bool bIsPaused = AudioComponent->GetPlayState() == EAudioComponentPlayState::Paused;
	const bool bPropagatePercentage = !bIsStopped && !bIsPaused;
	LastReceivedPlaybackPercent = InPlaybackPercentage;
	
	if (InComponent == AudioComponent && bPropagatePercentage)
	{
		if (TransportCoordinator.IsValid())
		{
			const float ClampedPlayBackPercentage = FGenericPlatformMath::Fmod(InPlaybackPercentage, 1.f);
			TransportCoordinator->SetProgressRatio(ClampedPlayBackPercentage);
		}
	}
}

void FWaveformEditor::HandleAudioComponentPlayStateChanged(const UAudioComponent* InAudioComponent, EAudioComponentPlayState NewPlayState)
{
	if (InAudioComponent != AudioComponent)
	{
		return;
	}

	switch (NewPlayState)
	{
	default:
		break;
	case EAudioComponentPlayState::Stopped:
		if (!TransportCoordinator->IsScrubbing() && !TransportCoordinator->IsScrolling())
		{
			// USoundWave::PostEditChangeProperty calls USoundWave::UpdateAsset which frees the soundwave's resources,
			// stopping the sound, calling this handler.  Avoiding this implicit state change is currently too risky and
			// so we clear the AudioComponent's paused state instead to keep it in sync with the USoundWave.
			AudioComponent->SetPaused(false);

			if (!bIsInteractingWithTransformations)
			{
				const float CachedAudioStartTimeAsPercentage = TransportController->GetCachedAudioStartTimeAsPercentage();
				TransportCoordinator->SetProgressRatio(CachedAudioStartTimeAsPercentage); // show cached start time for next play rather than playhead stop point (doesn't affect pause)
			}
		}
		break;
	}
}

void FWaveformEditor::HandleRenderDataUpdate()
{
	if (TransportCoordinator != nullptr)
	{
		TransportCoordinator->UpdatePlaybackRange(WaveformView.DataProvider->GetTransformedWaveformBounds());
		WaveformView.DataProvider->RequestSequenceView(TransportCoordinator->GetDisplayRange());
		WaveformView.ViewWidget->ResolveFromFramePosition();
	}
}

void FWaveformEditor::HandleDisplayRangeUpdate(const TRange<double> NewRange)
{
	WaveformView.DataProvider->RequestSequenceView(NewRange);
	WaveformView.ViewWidget->ResolveFromFramePosition();
}

void FWaveformEditor::HandlePlayheadScrub(const float InTargetPlayBackRatio, const bool bIsMoving)
{
	if (bIsMoving)
	{
		if (TransportController->IsPlaying())
		{
			TransportController->Stop();
			bWasPlayingBeforeScrubbing = true;
		}
	}
	else
	{
		float ActiveDuration = TransformationChainConfig.EndTime - TransformationChainConfig.StartTime;
		if (ActiveDuration <= 0.0f)
		{
			// If no active+initialized transformation, ActiveDuration will be <= 0.0f, so fallback to sound duration:
			ActiveDuration = SoundWave->Duration; // not GetDuration() to get the raw duration and not INDEFINITELY_LOOPING_DURATION if looping
		}
		const float ClampedTargetPlayBackRatio = FMath::Max(InTargetPlayBackRatio, 0.0f);
		const float NewTime = FGenericPlatformMath::Fmod(ClampedTargetPlayBackRatio, 1.0f) * ActiveDuration;

		PlaybackTimeBeforeTransformInteraction = NewTime;
		StartTimeBeforeTransformInteraction = TransformationChainConfig.StartTime;

		if (TransportController->IsPlaying())
		{
			TransportController->Seek(NewTime);
			return;
		}

		if (bWasPlayingBeforeScrubbing)
		{
			TransportController->Play(NewTime);
			bWasPlayingBeforeScrubbing = false;
		}
		else
		{
			TransportController->CacheStartTime(NewTime);
		}
	}
}

void FWaveformEditor::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SoundWave);
	Collector.AddReferencedObject(AudioComponent);
}

FString FWaveformEditor::GetReferencerName() const
{
	return TEXT("FWaveformEditor");
}

bool FWaveformEditor::CanPressPlayButton() const
{
	return TransportController->CanPlay() && (TransportController->IsPaused() || !TransportController->IsPlaying());
}

bool FWaveformEditor::CreateWaveWriter()
{
	if (!ensure(SoundWave))
	{
		return false;
	}

	WaveWriter = MakeShared<FWaveformEditorWaveWriter>(SoundWave);
	return WaveWriter != nullptr;
}

const UWaveformEditorTransformationsSettings* FWaveformEditor::GetWaveformEditorTransformationsSettings() const
{
	const UWaveformEditorTransformationsSettings* WaveformEditorTransformationsSettings = GetDefault<UWaveformEditorTransformationsSettings>();
	check(WaveformEditorTransformationsSettings)

	return WaveformEditorTransformationsSettings;
}

FText FWaveformEditor::GetReimportButtonToolTip() const
{
	FText ReimportModeText;

	switch (ReimportMode)
	{
	case(EWaveEditorReimportMode::SelectFile):
		ReimportModeText = LOCTEXT("SelectFile", "Reimport from new file");
		break;
	case(EWaveEditorReimportMode::SameFile):
		ReimportModeText = LOCTEXT("SameFile", "Reimport from same file");
		break;
	case(EWaveEditorReimportMode::SameFileOverwrite):
		ReimportModeText = LOCTEXT("SameFileOverwrite", "Reimport from same file and overwrite transformations");
		break;
	default:
		static_assert(static_cast<int32>(EWaveEditorReimportMode::COUNT) == 3, "Possible missing switch case coverage for 'EWaveEditorReimportMode'");
		break;
	}

	return FText::Format(LOCTEXT("WaveformEditorReimportButtonTooltip", "{0}."), ReimportModeText);
}

FText FWaveformEditor::GetExportButtonToolTip() const
{

	FText ExportModeText;

	if (!WaveWriter)
	{
		return ExportModeText;
	}

	switch (WaveWriter->GetExportChannelsFormat())
	{
	case(WaveformEditorWaveWriter::EChannelFormat::Stereo):
		ExportModeText = LOCTEXT("ExportModeStereo", "stereo");
		break;
	case(WaveformEditorWaveWriter::EChannelFormat::Mono):
		ExportModeText = LOCTEXT("ExportModeMono", "mono");
		break;
	default:
		break;
	}

	return FText::Format(LOCTEXT("WaveformEditorExportButtonTooltip", "Exports the edited waveform to a {0} USoundWave asset."), ExportModeText);
}

#undef LOCTEXT_NAMESPACE
