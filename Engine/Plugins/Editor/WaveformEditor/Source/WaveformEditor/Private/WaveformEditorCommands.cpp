// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorCommands.h"

#define LOCTEXT_NAMESPACE "WaveformEditorCommands"

FWaveformEditorCommands::FWaveformEditorCommands() 
	: TCommands<FWaveformEditorCommands>(
		TEXT("WaveformEditor"), 
		NSLOCTEXT("Contexts", "WaveformEditor", "Waveform Editor"), 
		NAME_None, 
		"AudioStyleSet"
	)
{}

void FWaveformEditorCommands::RegisterCommands()
{
	UI_COMMAND(PlaySoundWave, "Play", "Play this sound wave", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(PauseSoundWave, "Pause", "Pause this sound wave", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(StopSoundWave, "Stop", "Stops playback and rewinds to beginning of file", EUserInterfaceActionType::Button, FInputChord(EKeys::Enter));
	UI_COMMAND(TogglePlayback, "TogglePlayback", "Toggles between play and pause", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar));
	UI_COMMAND(ReturnToStart, "Return To Start", "Sets playhead to start of waveform", EUserInterfaceActionType::Button, FInputChord(EKeys::SpaceBar, EModifierKey::Shift));

	UI_COMMAND(ZoomIn, "Zoom In", "Zooms into the sound wave", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ZoomOut, "Zoom Out", "Zooms out from the sound wave", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(ExportWaveform, "Export Waveform", "Exports the edited waveform to a new sound wave uasset", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ExportFormatMono, "Mono", "Sets the export format to mono", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ExportFormatStereo, "Stereo", "Sets the export format to stereo", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ReimportAsset, "Reimport Asset", "Reimports the inspected sound wave uasset", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ReimportModeSameFile, "Reimports asset via referenced .wav file", "Reimports asset from the original .wav file", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ReimportModeSameFileOverwriteTransformations, "Reimport asset via referenced .wav file and overwrite transformations", "Reimport from the original .wav file and overwrite transformations", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(ReimportModeNewFile, "Reimport from a selected .wav file", "Reimports asset from a selected .wav file", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(ToggleFadeIn, "Toggle Fade In", "Toggle adding Fade In Transformation onto the sound wave", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FadeInLinear, "Linear Fade In", "Fade In Curve type applied when Toggle Fade In pressed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FadeInExponential, "Exponential Fade In", "Fade In Curve type applied when Toggle Fade In pressed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FadeInLogarithmic, "Logarithmic Fade In", "Fade In Curve type applied when Toggle Fade In pressed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FadeInSigmoid, "Sigmoid Fade In", "Fade In S-Shaped Curve type applied when Toggle Fade In pressed", EUserInterfaceActionType::RadioButton, FInputChord());
	
	UI_COMMAND(ToggleFadeOut, "Toggle Fade Out", "Toggle adding Fade Out Transformation onto the sound wave", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(FadeOutLinear, "Linear Fade Out", "Fade Out Curve type applied when Toggle Fade Out pressed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FadeOutExponential, "Exponential Fade Out", "Fade Out Curve type applied when Toggle Fade Out pressed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FadeOutLogarithmic, "Logarithmic Fade Out", "Fade Out Curve type applied when Toggle Fade Out pressed", EUserInterfaceActionType::RadioButton, FInputChord());
	UI_COMMAND(FadeOutSigmoid, "Sigmoid Fade Out", "Fade Out S-Shaped Curve type applied when Toggle Fade Out pressed", EUserInterfaceActionType::RadioButton, FInputChord());

	UI_COMMAND(LeftBoundsIncrease, "Increase Left Loop Bounds", "Moves the start of the loop region earlier", EUserInterfaceActionType::Button, FInputChord(EKeys::Comma));
	UI_COMMAND(LeftBoundsDecrease, "Decrease Left Loop Bounds", "Moves the start of the loop region later", EUserInterfaceActionType::Button, FInputChord(EKeys::Period));
	UI_COMMAND(RightBoundsIncrease, "Increase Right Loop Bounds", "Moves the end of the loop region earlier", EUserInterfaceActionType::Button, FInputChord(EKeys::RightBracket));
	UI_COMMAND(RightBoundsDecrease, "Decrease Right Loop Bounds", "Moves the end of the loop region later", EUserInterfaceActionType::Button, FInputChord(EKeys::LeftBracket));

	UI_COMMAND(LeftBoundsNextZeroCrossing, "Increase Left Loop Bounds To Next Zero Crossing", "Moves the start of the loop region earlier", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Comma));
	UI_COMMAND(LeftBoundsPreviousZeroCrossing, "Decrease Left Loop Bounds To Previous Zero Crossing", "Moves the start of the loop region later", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Period));
	UI_COMMAND(RightBoundsNextZeroCrossing, "Increase Right Loop Bounds To Next Zero Crossing", "Moves the end of the loop region earlier", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::RightBracket));
	UI_COMMAND(RightBoundsPreviousZeroCrossing, "Decrease Right Loop Bounds To Previous Zero Crossing", "Moves the end of the loop region later", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::LeftBracket));

	UI_COMMAND(BoundsIncrementIncrease, "Increase Bound Increment", "Increases the increment to shift the loop bounds", EUserInterfaceActionType::Button, FInputChord(EKeys::Equals));
	UI_COMMAND(BoundsIncrementDecrease, "Decrease Bound Increment", "Increases the increment to shift the loop bounds", EUserInterfaceActionType::Button, FInputChord(EKeys::Hyphen));
	UI_COMMAND(SelectNextLoop, "Select Next Loop", "Selects the next loop marker", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
	UI_COMMAND(SelectPreviousLoop, "Select Previous Loop", "Selects the previous loop marker", EUserInterfaceActionType::Button, FInputChord(EKeys::B));

	UI_COMMAND(CreateMarker, "Create Marker", "Toggle adding Marker Cue Transformation onto the sound wave", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CreateLoopRegion, "Create Loop Region", "Toggle adding Marker Loop Region Transformation onto the sound wave", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(DeleteMarker, "Delete Marker Or Loop Region", "Delete a selected Marker Cue or Loop Region from the sound wave", EUserInterfaceActionType::Button, FInputChord(EKeys::Delete));

	UI_COMMAND(SkipToNextMarker, "Skip To Next Marker", "Skips the playhead to the next marker", EUserInterfaceActionType::Button, FInputChord(EKeys::G));
	UI_COMMAND(SkipToNextZeroCrossing, "Skip To Next Zero Crossing", "Skips the playhead to the next zero crossing", EUserInterfaceActionType::Button, FInputChord(EKeys::H));
}
	
const FWaveformEditorCommands& FWaveformEditorCommands::Get()
{
	return TCommands<FWaveformEditorCommands>::Get();
}

#undef LOCTEXT_NAMESPACE
