// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorDetailsCustomization.h"

#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "Sound/SoundWave.h"

namespace WaveformTransformationsDetails
{
	static const FLazyName TransformationsCategoryName("Waveform Processing");

	void GetUClassProperties(const UStruct* InClass, TArray<FProperty*>& OutProperties)
	{
		OutProperties.Empty();

		for (TFieldIterator<FProperty> PropertyIterator(InClass); PropertyIterator; ++PropertyIterator)
		{
			OutProperties.Add(*PropertyIterator);
		}
	}

	void BuildDetailsView(IDetailLayoutBuilder& DetailLayout)
	{
		TArray<FName> CategoryNames;
		DetailLayout.GetCategoryNames(CategoryNames);

		for (FName& CategoryName : CategoryNames)
		{
			if (CategoryName != WaveformTransformationsDetails::TransformationsCategoryName)
			{
				DetailLayout.HideCategory(CategoryName);
			}
		}

		IDetailCategoryBuilder& CategoryBuilder = DetailLayout.EditCategory("Waveform Processing");
		CategoryBuilder.InitiallyCollapsed(false);
		CategoryBuilder.RestoreExpansionState(true);
	}
}

void FWaveformTransformationsDetailsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{	
	WaveformTransformationsDetails::BuildDetailsView(DetailLayout);
}

void FWaveformTransformationsDetailsProvider::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	WaveformTransformationsDetails::BuildDetailsView(DetailLayout);
	CachedTransformationsHandle = DetailLayout.GetProperty("Transformations");
}

void FWaveformTransformationsDetailsProvider::GetHandlesForUObjectProperties(const TObjectPtr<UObject> InUObject, TArray<TSharedRef<IPropertyHandle>>& OutPropertyHandles)
{
	check(CachedTransformationsHandle->IsValidHandle());

	OutPropertyHandles.Empty();

	TArray<FProperty*> ObjProperties;
	TArray<const FName> PropertiesPaths;


	uint32 NumTransformations = 0;

	CachedTransformationsHandle->GetNumChildren(NumTransformations);
	TSharedPtr<IPropertyHandle> TargetHandle;

	for (uint32 TransformIndex = 0; TransformIndex < NumTransformations; ++TransformIndex)
	{
		TSharedPtr<IPropertyHandle> ChildHandle = CachedTransformationsHandle->GetChildHandle(TransformIndex);

		UObject* UObjPtr = nullptr;
		ChildHandle->GetValue(UObjPtr);
		
		if (UObjPtr && UObjPtr == InUObject.Get())
		{
			TargetHandle = ChildHandle;
			break;
		}

	}

	if (TargetHandle == nullptr)
	{
		return;
	}
		
	WaveformTransformationsDetails::GetUClassProperties(InUObject->GetClass(), ObjProperties);

	for (const FProperty* Property : ObjProperties)
	{
		TSharedPtr<IPropertyHandle> PropertyHandle = TargetHandle->GetChildHandle(Property->GetFName());
		if (PropertyHandle->IsValidHandle())
		{
			OutPropertyHandles.Add(PropertyHandle.ToSharedRef());
		}
	}
}
