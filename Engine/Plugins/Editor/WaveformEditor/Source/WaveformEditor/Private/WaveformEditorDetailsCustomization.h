// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IWaveformEditorDetailsProvider.h"
#include "UObject/WeakObjectPtr.h"

class IDetailLayoutBuilder;

class FWaveformTransformationsDetailsCustomization : public IDetailCustomization
{
public:	
	/** IDetailCustomization interface */
	void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};


class FWaveformTransformationsDetailsProvider : public IDetailCustomization, public IWaveformEditorDetailsProvider
{
public:
	/** IDetailCustomization interface */
	void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;


	/** IWaveformEditorDetailsProvider interface */
	virtual void GetHandlesForUObjectProperties(const TObjectPtr<UObject> InUObject, TArray<TSharedRef<IPropertyHandle>>& OutPropertyHandles) override;

private:
	TSharedPtr<class IPropertyHandle> CachedTransformationsHandle;
};