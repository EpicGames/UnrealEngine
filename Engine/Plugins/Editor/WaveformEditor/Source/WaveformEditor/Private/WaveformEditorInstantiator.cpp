// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorInstantiator.h"

#include "Algo/AnyOf.h"
#include "AssetDefinitionRegistry.h"
#include "AssetTypeActions/AssetDefinition_SoundBase.h"
#include "AudioEditorModule.h"
#include "ContentBrowserMenuContexts.h"
#include "EditorReimportHandler.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundWaveProcedural.h"
#include "SWaveformEditorMessageDialog.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "WaveformEditor.h"
#include "WaveformEditorLog.h"
#include "WaveformEditorModule.h"

#define LOCTEXT_NAMESPACE "WaveformEditorInstantiator"

namespace WaveEditorInstantiatorPrivate
{
	auto FilterUnwantedAssets = [](const FAssetData& AssetData) { return !AssetData.IsInstanceOf<USoundWaveProcedural>(); };
}

void FWaveformEditorInstantiator::ExtendContentBrowserSelectionMenu()
{
	UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(USoundWave::StaticClass());
	check(Menu);
	FToolMenuSection& ImportSection = Menu->FindOrAddSection("ImportedAssetActions");
	ImportSection.AddDynamicEntry("SoundWaveEditing_ReimportAndOverwriteTransformations", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
	{
		if (const UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>())
		{
			if (Algo::AnyOf(Context->SelectedAssets, WaveEditorInstantiatorPrivate::FilterUnwantedAssets))
			{
				const TAttribute<FText> Label = LOCTEXT("SoundWave_ReimportAndOverwriteTransformations", "Reimport And Overwrite Transformations");
				const TAttribute<FText> ToolTip = LOCTEXT("SoundWave_ReimportAndOverwriteTransformations_Tooltip", "Reimport And Overwrite Transformations");
				const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Reimport");
				const FToolMenuExecuteAction UIAction = FToolMenuExecuteAction::CreateSP(this, &FWaveformEditorInstantiator::ExecuteReImportAndOverwriteTransformations);

				InSection.AddMenuEntry("SoundWave_ReimportAndOverwriteTransformations", Label, ToolTip, Icon, UIAction);
			}
		}
		}));
}

void FWaveformEditorInstantiator::RegisterAsSoundwaveEditor()
{
	IAudioEditorModule& AudioEditorModule = FModuleManager::LoadModuleChecked<IAudioEditorModule>( "AudioEditor" );

	// Only register as default Sound Wave editor if not running in restricted mode
	if (!AudioEditorModule.IsRestrictedMode())
	{
		AudioEditorModule.SoundWaveEditorOpen.BindSP(this, &FWaveformEditorInstantiator::CreateWaveformEditor);
	}
}

void FWaveformEditorInstantiator::ExecuteReImportAndOverwriteTransformations(const FToolMenuContext& MenuContext)
{
	if (const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(MenuContext))
	{
		TArray<USoundWave*> SoundWavesToEdit = Context->LoadSelectedObjectsIf<USoundWave>(WaveEditorInstantiatorPrivate::FilterUnwantedAssets);
		TArray<UObject*> ObjectArray;

		for (USoundWave* SoundWave : SoundWavesToEdit)
		{
			UObject* SelectedAsset = CastChecked<UObject>(SoundWave);
			
			ObjectArray.Emplace(SelectedAsset);

			for (TObjectPtr<UWaveformTransformationBase> Transformation : SoundWave->Transformations)
			{
				Transformation->OverwriteTransformation();
			}
		}

		FReimportManager::Instance()->ValidateAllSourceFileAndReimport(ObjectArray);
	}
}

bool FWaveformEditorInstantiator::CanSoundWaveBeOpenedInEditor(const USoundWave* SoundWaveToEdit)
{
	bool bCanOpenWaveEditor = true;
	
	FText ErrorText = LOCTEXT("WaveformEditorOpeningError", "Could not open waveform editor for Selected SoundWave");

	if (SoundWaveToEdit == nullptr)
	{
		ErrorText = LOCTEXT("WaveformEditorOpeningError_NullSoundWave", "Could not open waveform editor. Selected SoundWave was null.");
		bCanOpenWaveEditor = false;
	}
	else
	{
		FText SoundWaveNameText = FText::FromString(*(SoundWaveToEdit->GetName()));
		
		if (SoundWaveToEdit->GetDuration() == 0.f)
		{
			ErrorText = FText::Format(LOCTEXT("WaveformEditorOpeningError_ZeroDuration", "Could not open waveform editor for SoundWave '{0}': duration is 0"), SoundWaveNameText);
			bCanOpenWaveEditor = false;
		}

		if (SoundWaveToEdit->NumChannels == 0)
		{
			ErrorText = FText::Format(LOCTEXT("WaveformEditorOpeningError_ZeroChannels", "Could not open waveform editor for SoundWave '{0}': channel count is 0"), SoundWaveNameText);
			bCanOpenWaveEditor = false;
		}

		if (SoundWaveToEdit->TotalSamples == 0)
		{
			ErrorText = FText::Format(LOCTEXT("WaveformEditorOpeningError_ZeroSamples", "Could not open waveform editor for SoundWave '{0}': found 0 total samples.\n\nConsider reimporting the asset to fix it."), SoundWaveNameText);
			bCanOpenWaveEditor = false;
		}
	}


	if (!bCanOpenWaveEditor)
	{
		DisplayErrorDialog(ErrorText);
	}
	
	return bCanOpenWaveEditor;
}

void FWaveformEditorInstantiator::DisplayErrorDialog(const FText& ErrorMessage) const
{
	UE_LOGF(LogWaveformEditor, Warning, "%ls", *ErrorMessage.ToString())

	TSharedPtr<SWindow> OpeningErrorWindow = SNew(SWindow)
		.Title(LOCTEXT("WaveEditorErrorWindowTitle", "Waveform Editor"))
		.HasCloseButton(true)
		.SupportsMaximize(false)
		.SupportsMinimize(false)
		.SizingRule(ESizingRule::Autosized);

	OpeningErrorWindow->SetContent(SNew(SWaveformEditorMessageDialog).ParentWindow(OpeningErrorWindow).MessageToDisplay(ErrorMessage));

	TSharedPtr<SWindow> RootWindow = FGlobalTabmanager::Get()->GetRootWindow();

	if (RootWindow.IsValid())
	{
		FSlateApplication::Get().AddModalWindow(OpeningErrorWindow.ToSharedRef(), RootWindow);
	}
	else
	{
		FSlateApplication::Get().AddWindow(OpeningErrorWindow.ToSharedRef());
	}
}

void FWaveformEditorInstantiator::CreateWaveformEditor(TArray<USoundWave*> SoundWavesToEdit)
{
	for (USoundWave* SoundWavePtr : SoundWavesToEdit)
	{
		if (CanSoundWaveBeOpenedInEditor(SoundWavePtr))
		{
			TSharedPtr<FWaveformEditor> WaveformEditor = MakeShared<FWaveformEditor>();

			if (WaveformEditor == nullptr || !WaveformEditor->Init(EToolkitMode::Standalone, nullptr, SoundWavePtr))
			{
				UE_LOGF(LogWaveformEditor, Warning, "Could not open waveform editor for soundwave %ls, initialization failed", *(SoundWavePtr->GetName()));
				
				if (WaveformEditor)
				{
					WaveformEditor->CloseWindow(EAssetEditorCloseReason::AssetUnloadingOrInvalid);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE