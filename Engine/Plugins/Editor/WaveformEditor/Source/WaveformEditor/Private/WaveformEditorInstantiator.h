// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWaveformEditorInstantiator.h"
#include "Templates/SharedPointer.h"

class USoundWave;
struct FToolMenuContext;

class FWaveformEditorInstantiator : public IWaveformEditorInstantiator,  public TSharedFromThis<FWaveformEditorInstantiator>
{
public:
	virtual void ExtendContentBrowserSelectionMenu() override;
	
	// Registers using a delegate as the default editor for soundwave assets without main engine referencing waveform plugin in Sound Wave Asset Definition
	virtual void RegisterAsSoundwaveEditor() override;
	virtual ~FWaveformEditorInstantiator() = default;

private:
	virtual void CreateWaveformEditor(TArray<USoundWave*> SoundWavesToEdit) override;
	void ExecuteReImportAndOverwriteTransformations(const FToolMenuContext& MenuContext);
	bool CanSoundWaveBeOpenedInEditor(const USoundWave* SoundWaveToEdit);
	void DisplayErrorDialog(const FText& ErrorMessage) const;
};