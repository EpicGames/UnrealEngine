// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorModule.h"

#include "DSP/FloatArrayMath.h"
#include "Editor.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "AssetRegistry/IAssetRegistry.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "Sound/SoundWave.h"
#include "Sound/SoundWaveProcedural.h"
#include "TransformedWaveformViewFactory.h"
#include "WaveformAudioAnalysisFunctions.h"
#include "WaveformEditorCommands.h"
#include "WaveformEditorInstantiator.h"
#include "WaveformEditorLog.h"
#include "WaveformEditorTransformationsSettings.h"
#include "WaveformEditorTransformationsSettingsCustomization.h"

#include "UObject/PackageReload.h"

DEFINE_LOG_CATEGORY(LogWaveformEditor);

namespace WaveformAnalysis
{
	// Analyzes loudness of a sound wave and sets LUFS/SamplePeakDB values.
	bool AnalyzeSoundWaveLoudness(UObject* Object, bool bMarkDirty)
	{
		if (Object && Object->IsA<USoundWave>() && !Object->IsA<USoundWaveProcedural>())
		{
			TObjectPtr<USoundWave> SoundWave = CastChecked<USoundWave>(Object);

			TArray<uint8> ImportedRawPCMData;
			uint32 BufferSampleRate = 0;
			uint16 ImportedNumChannels = 0;

			Audio::FAlignedFloatBuffer AudioBuffer;

			if (!SoundWave->GetImportedSoundWaveData(ImportedRawPCMData, BufferSampleRate, ImportedNumChannels) || BufferSampleRate == 0)
			{
				UE_LOGF(LogWaveformEditor, Warning, "Soundwave loudness analyzation failed, could not import raw PCM data.");

				return false;
			}

			const int64 NumWaveformSamples = ImportedRawPCMData.Num() * sizeof(uint8) / sizeof(int16);
			AudioBuffer.SetNumUninitialized(NumWaveformSamples);
			Audio::ArrayPcm16ToFloat(MakeArrayView((int16*)ImportedRawPCMData.GetData(), NumWaveformSamples), AudioBuffer);

			const float SamplePeakDB = WaveformAudioAnalysis::GetPeakSampleValue(AudioBuffer);
			const float LUFS = WaveformAudioAnalysis::GetLUFS(AudioBuffer, BufferSampleRate, ImportedNumChannels);

			SoundWave->SetLoudnessValues(LUFS, SamplePeakDB);

			// Refresh the asset registry tags so the content browser tooltip displays updated metadata.
			// This avoids PostEditChangeProperty which would trigger an unnecessary derived data rebuild.
			IAssetRegistry::GetChecked().AssetUpdateTags(SoundWave, EAssetRegistryTagsCaller::FullUpdate);

			// Optionally mark the file dirty
			if (bMarkDirty)
			{
				SoundWave->MarkPackageDirty();
			}
		}
		else
		{
			return false;
		}

		return true;
	}
}

void FWaveformEditorModule::StartupModule()
{
	FWaveformEditorCommands::Register();
	FTransformedWaveformViewFactory::Create();

	WaveformEditorInstantiator = MakeShared<FWaveformEditorInstantiator>();
	RegisterContentBrowserExtensions(WaveformEditorInstantiator.Get());
	WaveformEditorInstantiator->RegisterAsSoundwaveEditor();

	if (FPropertyEditorModule* PropertyEditorModule = FModuleManager::LoadModulePtr<FPropertyEditorModule>("PropertyEditor"))
	{
		PropertyEditorModule->RegisterCustomClassLayout(
			UWaveformEditorTransformationsSettings::StaticClass()->GetFName(),
			FOnGetDetailCustomizationInstance::CreateStatic(&FWaveformEditorTransformationsSettingsCustomization::MakeInstance));

		PropertyEditorModule->NotifyCustomizationModuleChanged();
	}

	OnPostEngineInitHandle = FCoreDelegates::GetOnPostEngineInit().AddLambda([this]()
		{
			if (GEditor)
			{
				if (TObjectPtr<UImportSubsystem> ImportSubsystem = GEditor->GetEditorSubsystem<UImportSubsystem>())
				{
					OnAssetPostImportHandle = ImportSubsystem->OnAssetPostImport.AddLambda([&](UFactory* Factory, UObject* Object)
						{
							WaveformAnalysis::AnalyzeSoundWaveLoudness(Object);
						});
				}

				OnSoundBasePreviewHandle = GEditor->OnSoundBasePreview.AddLambda([](TObjectPtr<USoundBase> SoundBase)
					{
						if (SoundBase && SoundBase->IsA<USoundWave>())
						{
							TObjectPtr<USoundWave> SoundWave = CastChecked<USoundWave>(SoundBase);

							if (SoundWave->LUFS == 0.f && SoundWave->SamplePeakDB == 0.f)
							{
								// Pass bMarkDirty=false: previewing a sound should not dirty the package.
								WaveformAnalysis::AnalyzeSoundWaveLoudness(SoundBase, /*bMarkDirty=*/ false);
							}
						}
					});
			}

			OnPackageReloadedHandle = FCoreUObjectDelegates::OnPackageReloaded.AddLambda([](EPackageReloadPhase Phase, FPackageReloadedEvent* ReloadedEvent)
				{
					if (Phase == EPackageReloadPhase::PostPackageFixup)
					{
						check(ReloadedEvent);

						const UPackage* ReloadedPackage = ReloadedEvent->GetNewPackage();

						if (ReloadedPackage == nullptr)
						{
							return;
						}

						TArray<UObject*> ObjectsInPackage;
						GetObjectsWithOuter(ReloadedPackage, ObjectsInPackage, EGetObjectsFlags::IncludeNestedObjects);

						for (UObject* Object : ObjectsInPackage)
						{
							WaveformAnalysis::AnalyzeSoundWaveLoudness(Object);
						}
					}
				});
		});
}

void FWaveformEditorModule::ShutdownModule()
{
	FWaveformEditorCommands::Unregister();

	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyEditorModule.UnregisterCustomClassLayout(UWaveformEditorTransformationsSettings::StaticClass()->GetFName());
		PropertyEditorModule.NotifyCustomizationModuleChanged();
	}

	FCoreDelegates::GetOnPostEngineInit().Remove(OnPostEngineInitHandle);

	if (GEditor)
	{
		if (TObjectPtr<UImportSubsystem> ImportSubsystem = GEditor->GetEditorSubsystem<UImportSubsystem>())
		{
			ImportSubsystem->OnAssetPostImport.Remove(OnAssetPostImportHandle);
		}

		GEditor->OnSoundBasePreview.Remove(OnSoundBasePreviewHandle);
	}

	FCoreUObjectDelegates::OnPackageReloaded.Remove(OnPackageReloadedHandle);
}

void FWaveformEditorModule::RegisterContentBrowserExtensions(IWaveformEditorInstantiator* Instantiator)
{
	WaveformEditorInstantiator->ExtendContentBrowserSelectionMenu();
}

IMPLEMENT_MODULE(FWaveformEditorModule, WaveformEditor);
