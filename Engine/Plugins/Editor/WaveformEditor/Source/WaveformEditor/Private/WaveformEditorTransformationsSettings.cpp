// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorTransformationsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveformEditorTransformationsSettings)


FName UWaveformEditorTransformationsSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

#if WITH_EDITOR

FText UWaveformEditorTransformationsSettings::GetSectionText() const
{
	return NSLOCTEXT("WaveformEditorTransformations", "WaveformEditorTransformationsSettingsSection", "Waveform Editor Transformations");
}

FName UWaveformEditorTransformationsSettings::GetSectionName() const
{
	return TEXT("Waveform Editor Transformations");
}
#endif
