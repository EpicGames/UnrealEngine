// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Engine/DeveloperSettings.h"
#include "IWaveformTransformation.h"
#include "Templates/SubclassOf.h"

#include "WaveformEditorTransformationsSettings.generated.h"

/**
 * Settings to regulate Waveform Transformations behavior inside Waveform Editor plugin.
 */
UCLASS(config = EditorPerProjectUserSettings, defaultconfig, meta = (DisplayName = "Waveform Editor Transformations"))
class UWaveformEditorTransformationsSettings : public UDeveloperSettings
{
	GENERATED_BODY()

//~ Begin UDeveloperSettings interface
	virtual FName GetCategoryName() const;

#if WITH_EDITOR
	virtual FText GetSectionText() const override;
	virtual FName GetSectionName() const override;
#endif
	//~ End UDeveloperSettings interface
	
public:
	/** 
	 * A Transformation chain that will be added to the inspected Soundwave if there aren't any  
	 * Using TSet to limit each transformation type to one instance to reduce complexity and create a more consistent experience
	 * USoundWave::Transformations is a TArray because changing it to TSet will delete user data
	 * UWaveformTransformationTrimFade is hidden from selection via details customization because it is added automatically and removed if unused
	 */
	UPROPERTY(config, EditAnywhere, Category = "Launch Options")
	TSet<TSubclassOf<UWaveformTransformationBase>> LaunchTransformations;

	UPROPERTY(config, EditAnywhere, Category = "Zero Crossing Options", meta = (ClampMin = "-144.0", ClampMax = "0.0"))
	float NoiseThreshold = -60.0f;
};
