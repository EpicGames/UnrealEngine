// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorTransformationsSettingsCustomization.h"

#include "ClassViewerModule.h"
#include "DetailLayoutBuilder.h"
#include "IDetailsView.h"
#include "PropertyHandle.h"
#include "WaveformEditorTransformationsSettings.h"
#include "WaveformTransformationTrimFade.h"

namespace
{
	// True when the class picker was opened for the LaunchTransformations TSet (or one of its elements).
	bool IsForLaunchTransformationsProperty(const TSharedPtr<IPropertyHandle>& InHandle)
	{
		static const FName LaunchTransformationsName = GET_MEMBER_NAME_CHECKED(UWaveformEditorTransformationsSettings, LaunchTransformations);

		for (TSharedPtr<IPropertyHandle> Current = InHandle; Current.IsValid(); Current = Current->GetParentHandle())
		{
			if (const FProperty* Property = Current->GetProperty())
			{
				if (Property->GetFName() == LaunchTransformationsName)
				{
					return true;
				}
			}
		}
		return false;
	}
}

bool FWaveformTransformationLaunchClassFilter::IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs)
{
	if (!IsForLaunchTransformationsProperty(InInitOptions.PropertyHandle))
	{
		return true;
	}

	return !InClass->IsChildOf(UWaveformTransformationTrimFade::StaticClass());
}

bool FWaveformTransformationLaunchClassFilter::IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs)
{
	if (!IsForLaunchTransformationsProperty(InInitOptions.PropertyHandle))
	{
		return true;
	}

	return !InUnloadedClassData->IsChildOf(UWaveformTransformationTrimFade::StaticClass());
}

TSharedRef<IDetailCustomization> FWaveformEditorTransformationsSettingsCustomization::MakeInstance()
{
	return MakeShared<FWaveformEditorTransformationsSettingsCustomization>();
}

void FWaveformEditorTransformationsSettingsCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailLayout)
{
	const TSharedPtr<IDetailsView> DetailsView = DetailLayout.GetDetailsViewSharedPtr();
	if (!DetailsView.IsValid())
	{
		return;
	}

	TArray<TSharedRef<IClassViewerFilter>> Filters;

	// To gate this filter to only LaunchTransformations, we use IsForLaunchTransformationsProperty inside the filter
	Filters.Add(MakeShared<FWaveformTransformationLaunchClassFilter>());
	DetailsView->SetClassViewerFilters(Filters);
}
