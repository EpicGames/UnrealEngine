// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ClassViewerFilter.h"
#include "IDetailCustomization.h"
#include "Templates/SharedPointer.h"

class IDetailLayoutBuilder;

// Hides UWaveformTransformationTrimFade from the class viewer, since it is auto-applied and would be a no-op in LaunchTransformations.
class FWaveformTransformationLaunchClassFilter : public IClassViewerFilter
{
public:
	virtual bool IsClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const UClass* InClass, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override;
	virtual bool IsUnloadedClassAllowed(const FClassViewerInitializationOptions& InInitOptions, const TSharedRef<const IUnloadedBlueprintData> InUnloadedClassData, TSharedRef<FClassViewerFilterFuncs> InFilterFuncs) override;
};

// Installs the launch-transformation class filter onto the Waveform Editor Transformations settings panel, working around DisallowedClasses being ignored for TSet<TSubclassOf<>> properties.
class FWaveformEditorTransformationsSettingsCustomization : public IDetailCustomization
{
public:
	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailLayout) override;
};
