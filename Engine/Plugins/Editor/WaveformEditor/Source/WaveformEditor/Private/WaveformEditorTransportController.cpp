// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorTransportController.h"

#include "GenericPlatform/GenericPlatformMath.h"
#include "IWaveformTransformationRenderer.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundWave.h"

FWaveformEditorTransportController::FWaveformEditorTransportController(UAudioComponent* InAudioComponent)
	: AudioComponent(InAudioComponent)
{
	check(InAudioComponent != nullptr);
	AudioComponentPlayStateChangedDelegateHandle = InAudioComponent->OnAudioPlayStateChangedNative.AddRaw(this, &FWaveformEditorTransportController::HandleAudioComponentPlayStateChanged);
}

FWaveformEditorTransportController::~FWaveformEditorTransportController()
{
	Stop();
	check(AudioComponent != nullptr);
	AudioComponent->OnAudioPlayStateChangedNative.Remove(AudioComponentPlayStateChangedDelegateHandle);
}

void FWaveformEditorTransportController::Play()
{
	if (!CanPlay())
	{
		return;
	}

	if (IsPaused())
	{
		AudioComponent->SetPaused(false);

		if (!bCachedTimeDuringPause)
		{
			return;
		}
		
	}

	AudioComponent->Play(CachedAudioStartTime);	
}

void FWaveformEditorTransportController::Play(const float StartTime)
{
	if (!CanPlay())
	{
		return;
	}

	if (IsPaused())
	{
		AudioComponent->SetPaused(false);
	}

	CacheStartTime(StartTime);
	AudioComponent->Play(CachedAudioStartTime);
}

void FWaveformEditorTransportController::Pause()
{
	AudioComponent->SetPaused(true);
}


void FWaveformEditorTransportController::Stop()
{
	if (!CanStop())
	{
		return;
	}

	AudioComponent->Stop();

	if (IsPaused())
	{
		AudioComponent->SetPaused(false);
	}

	bCachedTimeDuringPause = false;
}

void FWaveformEditorTransportController::TogglePlayback()
{
	if (IsPlaying())
	{
		Pause();
	}
	else
	{
		Play();
	}
}

bool FWaveformEditorTransportController::CanPlay() const
{
	return SoundBaseIsValid();
}

bool FWaveformEditorTransportController::CanStop() const
{
	return (IsPlaying() || IsPaused());
}

bool FWaveformEditorTransportController::IsPaused() const
{
	return AudioComponent->GetPlayState() == EAudioComponentPlayState::Paused;
}

bool FWaveformEditorTransportController::IsPlaying() const
{
	return AudioComponent->GetPlayState() == EAudioComponentPlayState::Playing;
}

void FWaveformEditorTransportController::CacheStartTime(const float StartTime)
{
	ensure(StartTime >= 0.0f);
	CachedAudioStartTime = StartTime;

	if (IsPaused())
	{
		bCachedTimeDuringPause = true;
	}
	else
	{
		bCachedTimeDuringPause = false;
	}
}

void FWaveformEditorTransportController::Seek(const float SeekTime)
{
	AudioComponent->Play(SeekTime);
}

float FWaveformEditorTransportController::GetCachedAudioStartTimeAsPercentage() const
{
	check(AudioComponent != nullptr);

	float CachedAudioStartTimeAsPercentage = 0.0f;
	
	if (AudioComponent->Sound != nullptr)
	{
		TObjectPtr<USoundWave> AudioComponentSoundWave = Cast<USoundWave>(AudioComponent->Sound);
		if (AudioComponentSoundWave) 
		{
			const FWaveTransformUObjectConfiguration& SoundWaveTransformationChainConfig = AudioComponentSoundWave->GetTransformationChainConfig();
			float ActiveDuration = SoundWaveTransformationChainConfig.EndTime - SoundWaveTransformationChainConfig.StartTime;
			if (ActiveDuration <= 0.0f)
			{
				// If no active+initialized transformation, ActiveDuration will be <= 0.0f, so fallback to sound duration:
				ActiveDuration = AudioComponent->Sound->Duration; // not GetDuration() to get the raw duration and not INDEFINITELY_LOOPING_DURATION if looping
			}
			if (ActiveDuration > 0.0f)
			{
				ensure(CachedAudioStartTime >= 0.0f);
				const float PlaybackPercentage = CachedAudioStartTime / ActiveDuration;
				CachedAudioStartTimeAsPercentage = FGenericPlatformMath::Fmod(PlaybackPercentage, 1.0f);
			}
		}
	}
	
	return CachedAudioStartTimeAsPercentage;
}

const bool FWaveformEditorTransportController::SoundBaseIsValid() const
{
	return AudioComponent->GetSound() != nullptr;
}

void FWaveformEditorTransportController::HandleAudioComponentPlayStateChanged(const UAudioComponent* InAudioComponent, EAudioComponentPlayState NewPlayState)
{
	ensure(InAudioComponent == AudioComponent);

	switch (NewPlayState)
	{
	default:
		break;
	case EAudioComponentPlayState::Stopped:
		bCachedTimeDuringPause = false;
		break;
	}
}
