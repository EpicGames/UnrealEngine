// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorWaveWriter.h"

#include "AssetToolsModule.h"
#include "DSP/FloatArrayMath.h"
#include "EditorAssetLibrary.h"
#include "FileHelpers.h"
#include "Sound/SoundWave.h"
#include "WaveformEditorLog.h"

FWaveformEditorWaveWriter::FWaveformEditorWaveWriter(USoundWave* InSoundWave)
	: SourceSoundWave(InSoundWave)
	, WaveWriter(MakeUnique<Audio::FSoundWavePCMWriter>())
	, TargetChannelsFormat(GetSupportedFormatFromChannelCount(InSoundWave->NumChannels))
{
}

bool FWaveformEditorWaveWriter::CanCreateSoundWaveAsset() const
{
	return SourceSoundWave != nullptr && WaveWriter != nullptr && WaveWriter->IsDone();
}

void FWaveformEditorWaveWriter::ExportTransformedWaveform(bool bIsAsync)
{
	check(SourceSoundWave);
	check(WaveWriter);

	const FString DefaultSuffix = TEXT("_Edited");

	FString AssetName;
	FString PackageName;

	FAssetToolsModule& AssetToolsModule = FModuleManager::Get().LoadModuleChecked<FAssetToolsModule>("AssetTools");
	AssetToolsModule.Get().CreateUniqueAssetName(SourceSoundWave->GetOutermost()->GetName(), DefaultSuffix, PackageName, AssetName);
	
	FString AssetPath = FPaths::GetPath(PackageName);

	//The wave writer will already be putting 'Game' in front of the provided asset path
	AssetPath = AssetPath.Replace(TEXT("/Game"), TEXT(""), ESearchCase::CaseSensitive); 

	Audio::TSampleBuffer<> BufferToWrite = GenerateSampleBuffer();

	TFunction<void(const USoundWave*)> OnSoundWaveWritten = [AssetName, AssetPath](const USoundWave* ResultingWave) {
		UE_LOGF(LogWaveformEditor, Log, "Finished Exporting edited soundwave %ls/%ls", *AssetPath, *AssetName);
		TArray<UPackage*> PackagesToSave;

		if (ResultingWave == nullptr)
		{
			UE_LOGF(LogWaveformEditor, Warning, "Resulting Exported soundwave is null: %ls/%ls", *AssetPath, *AssetName);

			return;
		}

		if (!ResultingWave->IsValidLowLevel())
		{
			UE_LOGF(LogWaveformEditor, Warning, "Resulting Exported soundwave is invalid: %ls/%ls", *AssetPath, *AssetName);

			return;
		}

		if (UPackage* SoundWavePackage = ResultingWave->GetPackage())
		{
			PackagesToSave.Add(SoundWavePackage);

			SoundWavePackage->SetDirtyFlag(true);
			const FString PackageFileName = FPackageName::LongPackageNameToFilename(SoundWavePackage->GetName(), FPackageName::GetAssetPackageExtension());
			const bool bSuccess = UEditorLoadingAndSavingUtils::SavePackages(PackagesToSave, false);

			if (bSuccess)
			{
				UE_LOGF(LogWaveformEditor, Log, "Exported soundwave saving succeeded: % ls / % ls", *AssetPath, *AssetName);
			}
			else
			{
				UE_LOGF(LogWaveformEditor, Warning, "Exported soundwave saving failed: %ls/%ls", *AssetPath, *AssetName);
			}
		}
	};
	
	if (bIsAsync)
	{
		if (!WaveWriter->BeginWriteToSoundWave(AssetName, BufferToWrite, AssetPath, OnSoundWaveWritten, SourceSoundWave))
		{
			UE_LOGF(LogWaveformEditor, Log, "Exporting async edited soundwave to %ls/%ls failed", *AssetPath, *AssetName);
		}
	}
	else
	{
		if (!WaveWriter->SynchronouslyWriteSoundWave(BufferToWrite, &AssetName, &AssetPath, SourceSoundWave))
		{
			UE_LOGF(LogWaveformEditor, Log, "Exporting edited soundwave to %ls/%ls failed", *AssetPath, *AssetName);
		}
	}
}

WaveformEditorWaveWriter::EChannelFormat FWaveformEditorWaveWriter::GetExportChannelsFormat() const
{
	return TargetChannelsFormat;
}

void FWaveformEditorWaveWriter::SetExportChannelsFormat(const WaveformEditorWaveWriter::EChannelFormat InTargetChannelFormat)
{
	TargetChannelsFormat = InTargetChannelFormat;
}

WaveformEditorWaveWriter::EChannelFormat FWaveformEditorWaveWriter::GetSupportedFormatFromChannelCount(const int InChannelCount) const
{
	WaveformEditorWaveWriter::EChannelFormat OutSupportedFormat = DefaultExportFormat;
	
	switch (InChannelCount)
	{
	case 1:
		OutSupportedFormat = WaveformEditorWaveWriter::EChannelFormat::Mono;
		break;
	case 2:
		OutSupportedFormat = WaveformEditorWaveWriter::EChannelFormat::Stereo; 
		break;
	default:
		break;
	}

	return OutSupportedFormat;
}

Audio::TSampleBuffer<int16> FWaveformEditorWaveWriter::GenerateSampleBuffer() const
{
	TArray<uint8> RawPCMData;
	uint16 NumChannels;
	uint32 SampleRate;

	if (!SourceSoundWave->GetImportedSoundWaveData(RawPCMData, SampleRate, NumChannels))
	{
		UE_LOGF(LogInit, Warning, "Failed to get imported soundwave data for file: %ls. Edited waveform will not be rendered.", *SourceSoundWave->GetPathName());
		return Audio::TSampleBuffer<>();
	}

	uint32 NumSamples = RawPCMData.Num() * sizeof(uint8) / sizeof(int16);

	Audio::FAlignedFloatBuffer Buffer;
	Buffer.SetNumUninitialized(NumSamples);

	Audio::ArrayPcm16ToFloat(MakeArrayView((int16*)RawPCMData.GetData(), NumSamples), Buffer);

	if (SourceSoundWave->Transformations.Num() > 0)
	{
		Audio::FWaveformTransformationWaveInfo TransformationInfo;

		TransformationInfo.Audio = &Buffer;
		TransformationInfo.NumChannels = NumChannels;
		TransformationInfo.SampleRate = SampleRate;

		TArray<Audio::FTransformationPtr> Transformations = SourceSoundWave->CreateTransformations();

		for (const Audio::FTransformationPtr& Transformation : Transformations)
		{
			Transformation->ProcessAudio(TransformationInfo);
		}

		NormalizeBufferToValue(Buffer, 1.f);

		SampleRate = TransformationInfo.SampleRate;
		NumChannels = TransformationInfo.NumChannels;
		NumSamples = Buffer.Num();

		check(NumChannels > 0);
		check(SampleRate > 0);
	}

	if (GetSupportedFormatFromChannelCount(NumChannels) == TargetChannelsFormat)
	{
		Audio::TSampleBuffer<> OutBuffer(Buffer, NumChannels, SampleRate);
		return MoveTemp(OutBuffer);
	}

		
	switch (TargetChannelsFormat)
	{
	case WaveformEditorWaveWriter::EChannelFormat::Mono:
		return DownmixBufferToMono(Buffer, NumChannels, SampleRate);
	case WaveformEditorWaveWriter::EChannelFormat::Stereo:
		return UpmixBufferToStereo(Buffer, NumChannels, SampleRate);
	default:
		Audio::TSampleBuffer<int16> OutBuffer(Buffer, NumChannels, SampleRate);
		return MoveTemp(OutBuffer);
	}	
}

Audio::TSampleBuffer<int16> FWaveformEditorWaveWriter::DownmixBufferToMono(const Audio::FAlignedFloatBuffer& InSampleBuffer, const uint32 InNumChannels, const uint32 InSampleRate) const
{
	check(InNumChannels > 0)

	if (InNumChannels < 2)
	{
		Audio::TSampleBuffer<int16> OutBuffer(InSampleBuffer, InNumChannels, InSampleRate);
		return MoveTemp(OutBuffer);
	}

	Audio::TSampleBuffer<float> FloatSampleBuffer(InSampleBuffer, InNumChannels, InSampleRate);
	FloatSampleBuffer.MixBufferToChannels(1);
	Audio::FAlignedFloatBuffer MonoBuffer;
	MonoBuffer.SetNumUninitialized(FloatSampleBuffer.GetNumSamples());
	MonoBuffer = FloatSampleBuffer.GetArrayView();
	NormalizeBufferToValue(MonoBuffer, 1.f);

	Audio::TSampleBuffer<int16> OutMonoBuffer = Audio::TSampleBuffer<int16>(MonoBuffer, 1, FloatSampleBuffer.GetSampleRate());

	return MoveTemp(OutMonoBuffer);
}

Audio::TSampleBuffer<int16> FWaveformEditorWaveWriter::UpmixBufferToStereo(const Audio::FAlignedFloatBuffer& InSampleBuffer, const uint32 InNumChannels, const uint32 InSampleRate) const
{
	check(InNumChannels > 0)

	if (InNumChannels > 1)
	{
		Audio::TSampleBuffer<int16> OutBuffer(InSampleBuffer, InNumChannels, InSampleRate);
		return MoveTemp(OutBuffer);
	}

	Audio::FAlignedFloatBuffer StereoBuffer;
	StereoBuffer.SetNumUninitialized(InSampleBuffer.Num() * 2);

	for (int32 i = 0; i < InSampleBuffer.Num(); ++i)
	{
		const uint32 StereoFrameIndex = i * 2;
		StereoBuffer[StereoFrameIndex] = InSampleBuffer[i];
		StereoBuffer[StereoFrameIndex + 1] = InSampleBuffer[i];
	}

	NormalizeBufferToValue(StereoBuffer, 1.f);

	Audio::TSampleBuffer<int16> OutBuffer(StereoBuffer, 2, InSampleRate);
	return MoveTemp(OutBuffer);
}

void FWaveformEditorWaveWriter::NormalizeBufferToValue(Audio::FAlignedFloatBuffer& InOutBuffer, const float InTargetMaxValue) const
{
	const float MaxValue = Audio::ArrayMaxAbsValue(InOutBuffer);

	if (MaxValue > InTargetMaxValue)
	{
		Audio::ArrayMultiplyByConstantInPlace(InOutBuffer, InTargetMaxValue / MaxValue);
	}
}