// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sound/SampleBufferIO.h"

class USoundWave;

namespace WaveformEditorWaveWriter
{
	enum class EChannelFormat : int32
	{
		Mono = 1,
		Stereo
	};
}


class FWaveformEditorWaveWriter
{
public:
	explicit FWaveformEditorWaveWriter(USoundWave* InSoundWave);

	bool CanCreateSoundWaveAsset() const;

	// bIsAsync allows the editor to not hang per file say on bulk operations vs. specific cases that must have sync behaviour like tests
	void ExportTransformedWaveform(bool bIsAsync);

	WaveformEditorWaveWriter::EChannelFormat GetExportChannelsFormat() const;
	void SetExportChannelsFormat(const WaveformEditorWaveWriter::EChannelFormat InTargetChannelFormat);

private: 
	WaveformEditorWaveWriter::EChannelFormat GetSupportedFormatFromChannelCount(const int InChannelCount) const;

	Audio::TSampleBuffer<int16> GenerateSampleBuffer() const;
	Audio::TSampleBuffer<int16> DownmixBufferToMono(const Audio::FAlignedFloatBuffer& InSampleBuffer, const uint32 InNumChannels, const uint32 InSampleRate) const;
	Audio::TSampleBuffer<int16> UpmixBufferToStereo(const Audio::FAlignedFloatBuffer& InSampleBuffer, const uint32 InNumChannels, const uint32 InSampleRate) const;

	void NormalizeBufferToValue(Audio::FAlignedFloatBuffer& InOutBuffer, const float InTargetMaxValue) const;

	const WaveformEditorWaveWriter::EChannelFormat DefaultExportFormat = WaveformEditorWaveWriter::EChannelFormat::Stereo;
	USoundWave* SourceSoundWave = nullptr;
	TUniquePtr<Audio::FSoundWavePCMWriter> WaveWriter = nullptr;
	WaveformEditorWaveWriter::EChannelFormat TargetChannelsFormat;
};