// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"


class USoundWave;
class IWaveformEditorInstantiator
{
public:
	virtual void ExtendContentBrowserSelectionMenu() = 0;
	virtual void RegisterAsSoundwaveEditor() = 0;

private:
	virtual void CreateWaveformEditor(TArray<USoundWave*> SoundWavesToEdit) = 0;
};