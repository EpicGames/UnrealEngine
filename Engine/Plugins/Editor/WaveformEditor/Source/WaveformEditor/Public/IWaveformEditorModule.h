// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"

class IWaveformEditorInstantiator;

class IWaveformEditorModule : public IModuleInterface
{
	/** Registers waveform editor asset actions. */
	virtual void RegisterContentBrowserExtensions(IWaveformEditorInstantiator* Instantiator) = 0;
};
