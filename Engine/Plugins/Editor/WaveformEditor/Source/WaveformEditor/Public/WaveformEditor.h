// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "EditorUndoClient.h"
#include "IWaveformTransformation.h"
#include "Misc/NotifyHook.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "WaveformEditorTransportController.h"
#include "WaveformEditorZoomController.h"
#include "WaveformTransformationTrimFade.h"
#include "TransformedWaveformView.h"
#include "WaveformTransformationMarkers.h"

#define UE_API WAVEFORMEDITOR_API

class IToolkitHost;
class SDockTab;
class UAudioComponent;
class USoundWave;
class UWaveformEditorTransformationsSettings;

class FWaveformEditor 
	: public FAssetEditorToolkit
	, public FEditorUndoClient
	, public FGCObject
	, public FNotifyHook
{
public:
	UE_API bool Init(const EToolkitMode::Type Mode, const TSharedPtr<IToolkitHost>& InitToolkitHost, USoundWave* SoundWaveToEdit);
	UE_API virtual ~FWaveformEditor();

	/** FAssetEditorToolkit interface */
	UE_API virtual bool CloseWindow(EAssetEditorCloseReason InCloseReason) override;
	UE_API virtual void RegisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	UE_API virtual void UnregisterTabSpawners(const TSharedRef<FTabManager>& TabManager) override;
	UE_API virtual FName GetEditorName() const override;
	UE_API virtual FName GetToolkitFName() const override;
	UE_API virtual FText GetBaseToolkitName() const override;
	UE_API virtual EVisibility GetVisibilityWhileAssetCompiling() const override;
	UE_API virtual FString GetWorldCentricTabPrefix() const override;
	UE_API virtual FLinearColor GetWorldCentricTabColorScale() const override;

	UE_API void OnAssetReimport(UObject* ReimportedObject, bool bSuccessfullReimport);

	/** FNotifyHook interface */
	UE_API void NotifyPreChange(class FEditPropertyChain* PropertyAboutToChange) override;
	UE_API void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, class FEditPropertyChain* PropertyThatChanged) override;

	/** FEditorUndo interface */
	UE_API void PostUndo(bool bSuccess) override;
	UE_API void PostRedo(bool bSuccess) override;
	UE_API bool MatchesContext(const FTransactionContext& InContext, const TArray<TPair<UObject*, FTransactionObjectEvent>>& TransactionObjectContexts) const override;
	UE_API virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;

	UE_API void ExportWaveform(bool bIsAsync);

	// Skips the playhead to the next marker
	UE_API void SkipToNextMarker();
	UE_API void ResetPlaybackToStart();

	TSharedPtr<class FSparseSampledSequenceTransportCoordinator> GetTransportCoordinator() { return TransportCoordinator; }
	const int64 GetPlayheadFramePosition() const { return WaveformView.ViewWidget.IsValid() ? WaveformView.ViewWidget->GetPlayheadFramePosition() : 0; }

	static UE_API const double EndTimeInvalid;

private:
	UE_API bool InitializeAudioComponent();
	UE_API bool CreateTransportController();
	UE_API bool InitializeZoom();
	UE_API bool BindDelegates();
	UE_API bool SetUpAssetReimport();
	UE_API void ExecuteReimport();
	UE_API void ExecuteOverwriteTransformations();

	/**	Sets the wave editor layout */
	UE_API const TSharedRef<FTabManager::FLayout> SetupStandaloneLayout();

	/**	Toolbar Setup */
	UE_API bool RegisterToolbar();
	UE_API bool BindCommands();
	UE_API TSharedRef<SWidget> GenerateFadeInOptionsMenu();
	UE_API TSharedRef<SWidget> GenerateFadeOutOptionsMenu();
	UE_API TSharedRef<SWidget> GenerateExportOptionsMenu();
	UE_API TSharedRef<SWidget> GenerateImportOptionsMenu();
	UE_API bool CanExecuteReimport() const;

	/**	Details tabs set up */
	UE_API TSharedRef<SDockTab> SpawnTab_Properties(const FSpawnTabArgs& Args);
	UE_API TSharedRef<SDockTab> SpawnTab_Transformations(const FSpawnTabArgs& Args);
	UE_API bool CreateDetailsViews();

	/**	Waveform view tab setup */
	UE_API TSharedRef<SDockTab> SpawnTab_WaveformDisplay(const FSpawnTabArgs& Args);
	UE_API bool CreateWaveformView();
	UE_API bool CreateTransportCoordinator();
	UE_API void BindWaveformViewDelegates(FWaveformEditorSequenceDataProvider& ViewDataProvider, STransformedWaveformViewPanel& ViewWidget);
	UE_API void RemoveWaveformViewDelegates(FWaveformEditorSequenceDataProvider& ViewDataProvider, STransformedWaveformViewPanel& ViewWidget);

	/** Playback delegates handlers */
	UE_API void HandlePlaybackPercentageChange(const UAudioComponent* InComponent, const USoundWave* InSoundWave, const float InPlaybackPercentage);
	UE_API void HandleAudioComponentPlayStateChanged(const UAudioComponent* InAudioComponent, EAudioComponentPlayState NewPlayState);
	UE_API void HandlePlayheadScrub(const float InTargetPlayBackRatio, const bool bIsMoving);

	/* Data View Delegates Handlers*/
	UE_API void HandleRenderDataUpdate();
	UE_API void HandleDisplayRangeUpdate(const TRange<double>);

	/** FGCObject overrides */
	UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	UE_API virtual FString GetReferencerName() const override;
	UE_API bool CanPressPlayButton() const;
	UE_API bool CreateWaveWriter();
	UE_API const UWaveformEditorTransformationsSettings* GetWaveformEditorTransformationsSettings() const;
	UE_API void AddDefaultTransformations();

	/** Transformation Functions*/
	UE_API void NotifyPostTransformationChange(const EPropertyChangeType::Type& PropertyChangeType = EPropertyChangeType::ArrayAdd);
	// Relies on there being only one TrimFade Transformation on the soundwave
	UE_API TObjectPtr<UWaveformTransformationTrimFade> GetTrimFadeTransformation() const;
	UE_API TObjectPtr<UWaveformTransformationTrimFade> GetOrAddTrimFadeTransformation(const bool bMarkModify = true);

	UE_API TObjectPtr<UWaveformTransformationMarkers> GetOrAddMarkerTransformation();
	// Relies on there being only one Marker Transformation on the soundwave
	UE_API TObjectPtr<UWaveformTransformationMarkers> GetMarkerTransformation() const;
	UE_API TObjectPtr<UWaveformTransformationMarkers> AddMarkerTransformation();

	// Get all active transformations and bind the waveform editor delegates
	UE_API void BindDelegatesForActiveTransformations();
	UE_API void UnBindDelegatesForActiveTransformations();

	UE_API void ToggleFadeIn();
	UE_API bool CanFadeIn();

	UE_API void ToggleFadeOut();
	UE_API bool CanFadeOut();
	UE_API void CreateMarker(bool bIsLoopRegion);
	UE_API void DeleteMarker();
	UE_API void SkipToNextZeroCrossing();
	UE_API void RegenerateTransformations();
	UE_API void UpdateTransformations(bool bMarkDirty = true);
	UE_API void UpdateRenderElements();
	UE_API void UpdateTransportState();

#if WITH_EDITOR
	// Update SoundWave LUFS and SamplePeak values
	UE_API void UpdateSoundWaveProperties(); 
#endif //WITH_EDITOR

	// SoundWave->PostEditChange() calls UpdateAsset() which kills all playback of the sound asset.
	// Calling SoundWave->PostEditChangeProperty(PropertyChangedEvent) with a valid property event avoids this.
	UE_API void NotifySoundWavePostEditChangeProperty(const FName& PropertyName, const EPropertyChangeType::Type PropertyChangeType = EPropertyChangeType::ValueSet);

	UE_API void ModifyMarkerLoopRegion(ELoopModificationControls Modification);
	UE_API void CycleMarkerLoopRegion(ELoopModificationControls Modification);

	UE_API void SetCuePointOrigin(TObjectPtr<UWaveformTransformationMarkers> MarkerTransformation);
	UE_API void PromptToChangeCuePointOrigin();

	FTransformedWaveformView  WaveformView;

	/** Exports the edited waveform to a new asset */
	TSharedPtr<class FWaveformEditorWaveWriter> WaveWriter = nullptr;

	/** Manages Transport info in waveform panel */
	TSharedPtr<class FSparseSampledSequenceTransportCoordinator> TransportCoordinator = nullptr;

	/** Controls Transport of the audio component  */
	TSharedPtr<FWaveformEditorTransportController> TransportController = nullptr;

	/** Controls and propagates zoom level */
	TSharedPtr<FWaveformEditorZoomController> ZoomManager = nullptr;

	/** Properties tab */
	TSharedPtr<IDetailsView> PropertiesDetails;

	/** Transformations tab */
	TSharedPtr<IDetailsView> TransformationsDetails;

	/** Settings Editor App Identifier */
	static UE_API const FName AppIdentifier;

	/** Tab Ids */
	static UE_API const FName PropertiesTabId;
	static UE_API const FName TransformationsTabId;
	static UE_API const FName WaveformDisplayTabId;
	static UE_API const FName EditorName;
	static UE_API const FName ToolkitFName;
	TObjectPtr<USoundWave> SoundWave = nullptr;
	TObjectPtr<UAudioComponent> AudioComponent = nullptr;
	bool bWasPlayingBeforeScrubbing = false;
	bool bIsInteractingWithTransformations = false;
	bool bWasPlayingBeforeChange = false;
	// Defer UpdateRenderElements onto a single ticker fire so heavy regen doesn't run every mouse-move.
	bool bDeferredRenderUpdatePending = false;
	// Handle to the in-flight one-shot ticker, kept so a non-interactive supersede or the destructor can cancel the pending fire.
	FTSTicker::FDelegateHandle DeferredRenderTickerHandle;
	float LastReceivedPlaybackPercent = 0.f;
	EAudioComponentPlayState TransformInteractionPlayState = EAudioComponentPlayState::Stopped;
	float PlaybackTimeBeforeTransformInteraction = 0.f;
	float StartTimeBeforeTransformInteraction = 0.f;
	FWaveTransformUObjectConfiguration TransformationChainConfig;

	static constexpr float DefaultFadeInAmount = 0.5f;
	float CachedFadeInAmount = 0.f;
	TSubclassOf<UTransformationFadeFunction> FadeInType = UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Exponential];

	static constexpr float DefaultFadeOutAmount = 0.5f;
	float CachedFadeOutAmount = 0.f;
	TSubclassOf<UTransformationFadeFunction> FadeOutType = UWaveformTransformationFade::FadeModeToFadeFunctionMap[EWaveEditorTransformationFadeMode::Exponential];

	// Tracking bool to ensure we detect wave cue changes and regenerate transformations
	// Checking the event is bound may have issues if other systems need to subscribe for wave cue array changes
	bool bCueChangeRegisteredByWaveformEditor = false;

	enum class EWaveEditorReimportMode : uint8
	{
		SameFile = 0,
		SameFileOverwrite,
		SelectFile,
		COUNT

	} ReimportMode;

	UE_API FText GetReimportButtonToolTip() const;
	UE_API FText GetExportButtonToolTip() const;

	TSharedPtr<SNotificationItem> ReopenNotificationItem = nullptr;
	TSharedPtr<SNotificationItem> ChangeCuePointOriginNotificationItem = nullptr;
	
	bool bMarkerTransformationIsActive = false;

	/** Set to true when the editor is being closed because the SoundWave asset is being deleted or invalidated.
	 *  Guards against unsafe SoundWave modification in the destructor cleanup path. */
	bool bSoundWaveBeingDeleted = false;
};

#undef UE_API
