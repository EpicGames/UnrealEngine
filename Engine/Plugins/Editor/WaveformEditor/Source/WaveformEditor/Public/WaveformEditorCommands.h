// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Framework/Commands/Commands.h"

class FWaveformEditorCommands : public TCommands<FWaveformEditorCommands>
{
public:
	FWaveformEditorCommands();

	virtual void RegisterCommands() override;
	static const FWaveformEditorCommands& Get();

	// Soundwave Playback controls
	TSharedPtr<FUICommandInfo> PlaySoundWave;
	TSharedPtr<FUICommandInfo> PauseSoundWave;
	TSharedPtr<FUICommandInfo> StopSoundWave;
	TSharedPtr<FUICommandInfo> TogglePlayback;
	TSharedPtr<FUICommandInfo> ReturnToStart;

	TSharedPtr<FUICommandInfo> ZoomIn;
	TSharedPtr<FUICommandInfo> ZoomOut;

	TSharedPtr<FUICommandInfo> ExportWaveform;
	TSharedPtr<FUICommandInfo> ExportFormatMono;
	TSharedPtr<FUICommandInfo> ExportFormatStereo;

	TSharedPtr<FUICommandInfo> ReimportAsset;
	TSharedPtr<FUICommandInfo> ReimportModeSameFile;
	TSharedPtr<FUICommandInfo> ReimportModeSameFileOverwriteTransformations;
	TSharedPtr<FUICommandInfo> ReimportModeNewFile;

	TSharedPtr<FUICommandInfo> ToggleFadeIn;
	TSharedPtr<FUICommandInfo> FadeInLinear;
	TSharedPtr<FUICommandInfo> FadeInExponential;
	TSharedPtr<FUICommandInfo> FadeInLogarithmic;
	TSharedPtr<FUICommandInfo> FadeInSigmoid;

	TSharedPtr<FUICommandInfo> ToggleFadeOut;
	TSharedPtr<FUICommandInfo> FadeOutLinear;
	TSharedPtr<FUICommandInfo> FadeOutExponential;
	TSharedPtr<FUICommandInfo> FadeOutLogarithmic;
	TSharedPtr<FUICommandInfo> FadeOutSigmoid;

	TSharedPtr<FUICommandInfo> LeftBoundsIncrease;
	TSharedPtr<FUICommandInfo> LeftBoundsDecrease;
	TSharedPtr<FUICommandInfo> RightBoundsIncrease;
	TSharedPtr<FUICommandInfo> RightBoundsDecrease;
	TSharedPtr<FUICommandInfo> LeftBoundsNextZeroCrossing;
	TSharedPtr<FUICommandInfo> LeftBoundsPreviousZeroCrossing;
	TSharedPtr<FUICommandInfo> RightBoundsNextZeroCrossing;
	TSharedPtr<FUICommandInfo> RightBoundsPreviousZeroCrossing;
	TSharedPtr<FUICommandInfo> BoundsIncrementIncrease;
	TSharedPtr<FUICommandInfo> BoundsIncrementDecrease;
	TSharedPtr<FUICommandInfo> SelectNextLoop;
	TSharedPtr<FUICommandInfo> SelectPreviousLoop;

	TSharedPtr<FUICommandInfo> CreateMarker;
	TSharedPtr<FUICommandInfo> CreateLoopRegion;
	TSharedPtr<FUICommandInfo> DeleteMarker;

	TSharedPtr<FUICommandInfo> SkipToNextMarker;
	TSharedPtr<FUICommandInfo> SkipToNextZeroCrossing;
};
