// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWaveformEditorModule.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

class IWaveformEditorInstantiator;

namespace WaveformAnalysis
{
	/** Analyzes loudness of a sound wave and sets LUFS/SamplePeakDB values. 
	 *  Returns true if a soundwave was analyzed.
	 */
	bool AnalyzeSoundWaveLoudness(UObject* Object, bool bMarkDirty = true);
}

class FWaveformEditorModule : public IWaveformEditorModule
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	virtual void RegisterContentBrowserExtensions(IWaveformEditorInstantiator* Instantiator) override;

	TSharedPtr<IWaveformEditorInstantiator> WaveformEditorInstantiator = nullptr;

	FDelegateHandle OnPostEngineInitHandle;
	FDelegateHandle OnAssetPostImportHandle;
	FDelegateHandle OnSoundBasePreviewHandle;
	FDelegateHandle OnPackageReloadedHandle;
};

