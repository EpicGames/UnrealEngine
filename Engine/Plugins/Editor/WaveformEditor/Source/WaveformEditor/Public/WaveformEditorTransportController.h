// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/AudioComponent.h"
#include "Delegates/IDelegateInstance.h"

class FWaveformEditorTransportController
{
public:
	explicit FWaveformEditorTransportController(UAudioComponent* InAudioComponent);
	~FWaveformEditorTransportController();

	void Play();
	void Play(const float StartTime);
	void Pause();
	void Stop();
	void TogglePlayback();
	bool CanPlay() const;
	bool CanStop() const;
	bool IsPaused() const;
	bool IsPlaying() const;
	void CacheStartTime(const float StartTime);
	void Seek(const float SeekTime);
	float GetCachedAudioStartTimeAsPercentage() const;

private:
	const bool SoundBaseIsValid() const;
	// Playback delegates handlers
	void HandleAudioComponentPlayStateChanged(const UAudioComponent* InAudioComponent, EAudioComponentPlayState NewPlayState);
	FDelegateHandle AudioComponentPlayStateChangedDelegateHandle;

	UAudioComponent* AudioComponent = nullptr;
	float CachedAudioStartTime = 0.f;
	bool bCachedTimeDuringPause = false;
};