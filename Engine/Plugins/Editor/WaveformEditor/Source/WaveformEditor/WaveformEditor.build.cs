// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class WaveformEditor : ModuleRules
	{
		public WaveformEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetRegistry",
					"AudioExtensions",
					"AudioSynesthesiaCore",
					"AudioWidgets",
					"Core",
					"CoreUObject",
					"ContentBrowser",
					"DeveloperSettings",
					"EditorScriptingUtilities",
					"EditorStyle",
					"Engine",
					"InputCore",
					"SignalProcessing",
					"Slate",
					"SlateCore",
					"ToolMenus",
					"UnrealEd",
					"WaveformEditorWidgets",
					"WaveformTransformations",
					"WaveformTransformationsWidgets"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"AssetDefinition",
					"AudioEditor",
					"ClassViewer",
					"PropertyEditor",
				}
			);
		}
	}
}
