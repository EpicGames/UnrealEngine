// Copyright Epic Games, Inc. All Rights Reserved.

 #include "STransformedWaveformViewPanel.h"

#include "DSP/Dsp.h"
#include "SampledSequenceDisplayUnit.h"
#include "SFixedSampledSequenceRuler.h"
#include "SFixedSampledSequenceViewer.h"
#include "SPlayheadOverlay.h"
#include "SSampledSequenceValueGridOverlay.h"
#include "SWaveformEditorInputRoutingOverlay.h"
#include "SWaveformTransformationsOverlay.h"
#include "WaveformEditorZoomController.h"
#include "WaveformEditorGridData.h"
#include "WaveformEditorStyle.h"
#include "WaveformEditorWidgetsSettings.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "TransformedWaveformViewPanel"

void STransformedWaveformViewPanel::Construct(const FArguments& InArgs, const FFixedSampledSequenceView& InData)
{
	DisplayUnit = ESampledSequenceDisplayUnit::Seconds;
	DataView = InData;

	WaveformEditorStyle = &FWaveformEditorStyle::Get();
	check(WaveformEditorStyle);

	SetUpGridData();
	SetUpBackground();
	SetUpWaveformViewer(GridData.ToSharedRef(), DataView);
	SetUpValueGridOverlay();

	if (InArgs._TransformationsOverlay)
	{
		WaveformTransformationsOverlay = InArgs._TransformationsOverlay;
	}

	if (InArgs._ZoomController)
	{
		ZoomController = InArgs._ZoomController;
	}

	SetupPlayheadOverlay();
	SetUpInputRoutingOverlay();
	SetUpTimeRuler(GridData.ToSharedRef());
	SetUpInputOverrides(InArgs);

	const UWaveformEditorWidgetsSettings* Settings = GetWaveformEditorWidgetsSettings();
	check(Settings)
	Settings->OnSettingChanged().AddSP(this, &STransformedWaveformViewPanel::OnWaveEditorWidgetSettingsUpdated);

	CreateLayout();
}

void STransformedWaveformViewPanel::CreateLayout()
{
	check(TimeRuler);
	check(WaveformViewer);
	check(InputRoutingOverlay);

	TSharedPtr<SOverlay> WaveformView = SNew(SOverlay);

	WaveformView->AddSlot()
	[
		BackgroundBorder.ToSharedRef()
	];

	WaveformView->AddSlot()
	[
		ValueGridOverlay.ToSharedRef()
	];

	WaveformView->AddSlot()
	[
		WaveformViewer.ToSharedRef()
	];

	if (WaveformTransformationsOverlay)
	{
		WaveformView->AddSlot()
		[
			WaveformTransformationsOverlay.ToSharedRef()
		];
	}

	WaveformView->AddSlot()
	[
		PlayheadOverlay.ToSharedRef()
	];

	WaveformView->AddSlot()
	[
		InputRoutingOverlay.ToSharedRef()
	];


	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot().AutoHeight()
		[
			TimeRuler.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		[
			WaveformView.ToSharedRef()
		]
	];
}

void STransformedWaveformViewPanel::SetUpTimeRuler(TSharedRef<FWaveformEditorGridData> InGridData)
{
	const FFixedSampleSequenceRulerStyle& TimeRulerStyle = WaveformEditorStyle->GetWidgetStyle<FFixedSampleSequenceRulerStyle>("WaveformEditorRuler.Style");

	TimeRuler = SNew(SFixedSampledSequenceRuler, InGridData).DisplayUnit(DisplayUnit).Style(&TimeRulerStyle);
	GridData->OnGridMetricsUpdated.AddSP(TimeRuler.ToSharedRef(), &SFixedSampledSequenceRuler::UpdateGridMetrics);
	WaveformEditorStyle->OnNewTimeRulerStyle.AddSP(TimeRuler.ToSharedRef(), &SFixedSampledSequenceRuler::OnStyleUpdated);
	TimeRuler->OnTimeUnitMenuSelection.AddSP(this, &STransformedWaveformViewPanel::UpdateDisplayUnit);
}

void STransformedWaveformViewPanel::SetUpInputRoutingOverlay()
{
	const FSampledSequenceViewerStyle& ViewerStyle = WaveformEditorStyle->GetWidgetStyle<FSampledSequenceViewerStyle>("WaveformViewer.Style");
	
	TArray<TSharedPtr<SWidget>> OverlaidWidgets;

	if (WaveformTransformationsOverlay)
	{
		OverlaidWidgets.Add(WaveformTransformationsOverlay);
	}

	check(PlayheadOverlay)
	OverlaidWidgets.Add(PlayheadOverlay);
	InputRoutingOverlay = SNew(SWaveformEditorInputRoutingOverlay, OverlaidWidgets).Style(&ViewerStyle);
}

void STransformedWaveformViewPanel::SetupPlayheadOverlay()
{
	const FPlayheadOverlayStyle& PlayheadOverlayStyle = WaveformEditorStyle->GetWidgetStyle<FPlayheadOverlayStyle>("WaveformEditorPlayheadOverlay.Style");
	PlayheadOverlay = SNew(SPlayheadOverlay).Style(&PlayheadOverlayStyle);
	WaveformEditorStyle->OnNewPlayheadOverlayStyle.AddSP(PlayheadOverlay.ToSharedRef(), &SPlayheadOverlay::OnStyleUpdated);
}

void STransformedWaveformViewPanel::SetUpWaveformViewer(TSharedRef<FWaveformEditorGridData> InGridData, const FFixedSampledSequenceView& InData)
{
	const FSampledSequenceViewerStyle& WaveViewerStyle = WaveformEditorStyle->GetWidgetStyle<FSampledSequenceViewerStyle>("WaveformViewer.Style");

	WaveformViewer = SNew(SFixedSampledSequenceViewer, InData.SampleData, InData.NumDimensions, InGridData).Style(&WaveViewerStyle).HideBackground(true);
		
	WaveformEditorStyle->OnNewWaveformViewerStyle.AddSP(WaveformViewer.ToSharedRef(), &SFixedSampledSequenceViewer::OnStyleUpdated);
	GridData->OnGridMetricsUpdated.AddSP(WaveformViewer.ToSharedRef(), &SFixedSampledSequenceViewer::UpdateGridMetrics);
}

void STransformedWaveformViewPanel::SetUpGridData()
{
	const FFixedSampleSequenceRulerStyle& RulerStyle = WaveformEditorStyle->GetWidgetStyle<FFixedSampleSequenceRulerStyle>("WaveformEditorRuler.Style");
	
	GridData = MakeShared<FWaveformEditorGridData>(DataView.SampleData.Num() / DataView.NumDimensions, DataView.SampleRate, RulerStyle.DesiredWidth, &RulerStyle.TicksTextFont);
}

void STransformedWaveformViewPanel::ReceiveSequenceView(const FFixedSampledSequenceView InView, const uint32 FirstSampleIndex)
{
	DataView = InView;
	if (GridData)
	{
		const uint32 FirstRenderedFrame = FirstSampleIndex / InView.NumDimensions;
		const uint32 NumFrames = InView.SampleData.Num() / InView.NumDimensions;
		GridData->UpdateDisplayRange(TRange<uint32>(FirstRenderedFrame, FirstRenderedFrame + NumFrames));
	}

	if (WaveformTransformationsOverlay)
	{
		WaveformTransformationsOverlay->UpdateLayerConstraints();
	}

	if (WaveformViewer)
	{
		WaveformViewer->UpdateView(InView.SampleData, InView.NumDimensions);
	}

	if (ValueGridOverlay)
	{
		ValueGridOverlay->ForceRedraw();
	}
}

void STransformedWaveformViewPanel::SetPlayheadRatio(const double InRatio)
{
	check(InRatio >= 0.0 && InRatio <= 1.0);
	CachedPlayheadRatio = InRatio;
	check(GridData);
	CachedPlayheadFramePosition = GridData->GetFramePositionFromRatio(InRatio);
}

void STransformedWaveformViewPanel::SetPlayheadFramePosition(const int64 FramePosition)
{
	CachedPlayheadFramePosition = FramePosition;
}

void STransformedWaveformViewPanel::ResolveFromFramePosition()
{
	check(GridData);
	CachedPlayheadRatio = GridData->GetRatioFromFramePosition(CachedPlayheadFramePosition);
}

void STransformedWaveformViewPanel::SetOnPlayheadOverlayMouseButtonUp(FPointerEventHandler InEventHandler)
{
	check(PlayheadOverlay)
	PlayheadOverlay->SetOnMouseButtonUp(InEventHandler);
}

void STransformedWaveformViewPanel::SetOnTimeRulerMouseButtonUp(FPointerEventHandler InEventHandler)
{
	check(TimeRuler)
	TimeRuler->SetOnMouseButtonUp(InEventHandler);
}

void STransformedWaveformViewPanel::SetOnTimeRulerMouseButtonDown(FPointerEventHandler InEventHandler)
{
	check(TimeRuler)
	TimeRuler->SetOnMouseButtonDown(InEventHandler);
}

void STransformedWaveformViewPanel::SetOnTimeRulerMouseMove(FPointerEventHandler InEventHandler)
{
	check(TimeRuler)
	TimeRuler->SetOnMouseMove(InEventHandler);
}

void STransformedWaveformViewPanel::SetOnMouseWheel(FPointerEventHandler InEventHandler)
{
	check(InputRoutingOverlay)
	InputRoutingOverlay->OnMouseWheelDelegate = InEventHandler;
}

FReply STransformedWaveformViewPanel::LaunchTimeRulerContextMenu()
{
	if (TimeRuler)
	{
		return TimeRuler->LaunchContextMenu();
	}

	return FReply::Unhandled();
}

void STransformedWaveformViewPanel::UpdateDisplayUnit(const ESampledSequenceDisplayUnit InDisplayUnit)
{
	DisplayUnit = InDisplayUnit;
	TimeRuler->UpdateDisplayUnit(DisplayUnit);
}

void STransformedWaveformViewPanel::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	const float PaintedWidth = AllottedGeometry.GetLocalSize().X;

	if (PaintedWidth != CachedPixelWidth)
	{
		CachedPixelWidth = PaintedWidth;

		if (GridData)
		{
			GridData->UpdateGridMetrics(PaintedWidth);
		}
	}

	if (PlayheadOverlay && GridData)
	{
		PlayheadOverlay->SetPlayheadVisible(GridData->IsFramePositionInDisplayRange(CachedPlayheadFramePosition));
	}

	if (TimeRuler && GridData)
	{
		TimeRuler->SetPlayheadVisible(GridData->IsFramePositionInDisplayRange(CachedPlayheadFramePosition));
	}

	UpdatePlayheadPosition(PaintedWidth);

	if (ZoomController)
	{
		ZoomController->CheckBounds(AllottedGeometry);
	}
}

void STransformedWaveformViewPanel::UpdatePlayheadPosition(const float PaintedWidth)
{
	// If the frame position is not visible then the overlays will not render and do not need to update
	if (GridData != nullptr && !GridData->IsFramePositionInDisplayRange(CachedPlayheadFramePosition))
	{
		return;
	}

	float PlayheadX = CachedPlayheadRatio * PaintedWidth;
	PlayheadX = GridData ? GridData->SnapPositionToClosestFrame(PlayheadX) : PlayheadX;

	if (PlayheadOverlay)
	{
		PlayheadOverlay->SetPlayheadPosition(PlayheadX);
	}

	if (TimeRuler)
	{
		TimeRuler->SetPlayheadPosition(PlayheadX);
	}
}

void STransformedWaveformViewPanel::UpdateBackground(const FSampledSequenceViewerStyle UpdatedStyle)
{
	BackgroundBorder->SetBorderBackgroundColor(UpdatedStyle.SequenceBackgroundColor);
}

void STransformedWaveformViewPanel::OnWaveEditorWidgetSettingsUpdated(const FName& PropertyName, const UWaveformEditorWidgetsSettings* Settings)
{
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, ShowLoudnessGrid))
	{
		if (ValueGridOverlay)
		{
			ValueGridOverlay->SetHideGrid(!Settings->ShowLoudnessGrid);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, ShowLoudnessGridDecibelValues))
	{
		if (ValueGridOverlay)
		{
			ValueGridOverlay->SetHideLabels(!Settings->ShowLoudnessGridDecibelValues);
		}
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, MaxLoudnessGridDivisions))
	{
		if (ValueGridOverlay)
		{
			ValueGridOverlay->SetMaxDivisionParameter(Settings->MaxLoudnessGridDivisions);
		}
	}
}

const UWaveformEditorWidgetsSettings* STransformedWaveformViewPanel::GetWaveformEditorWidgetsSettings()
{
	const UWaveformEditorWidgetsSettings* WaveformEditorWidgetsSettings = GetDefault<UWaveformEditorWidgetsSettings>();
	check(WaveformEditorWidgetsSettings);

	return WaveformEditorWidgetsSettings;
}

void STransformedWaveformViewPanel::SetUpBackground()
{
	WaveformEditorStyle->OnNewWaveformViewerStyle.AddSP(this, &STransformedWaveformViewPanel::UpdateBackground);

	const FSampledSequenceViewerStyle& ViewerStyle = WaveformEditorStyle->GetWidgetStyle<FSampledSequenceViewerStyle>("WaveformViewer.Style");

	BackgroundBorder = SNew(SBorder)
		.BorderImage(&ViewerStyle.BackgroundBrush)
		.BorderBackgroundColor(ViewerStyle.SequenceBackgroundColor);
}

void STransformedWaveformViewPanel::SetUpInputOverrides(const FArguments& InArgs)
{
	SetOnPlayheadOverlayMouseButtonUp(InArgs._OnPlayheadOverlayMouseButtonUp);

	SetOnTimeRulerMouseButtonUp(InArgs._OnTimeRulerMouseButtonUp);
	SetOnTimeRulerMouseButtonDown(InArgs._OnTimeRulerMouseButtonDown);
	SetOnTimeRulerMouseMove(InArgs._OnTimeRulerMouseMove);
	
	SetOnMouseWheel(InArgs._OnMouseWheel);
}

void STransformedWaveformViewPanel::SetUpValueGridOverlay()
{
	auto ValueGridDBConverter = [](const double InLabelValue)
	{
		
		const float AbsAmplitude = FMath::Abs(InLabelValue * 2.f - 1);
		const float GridLineValue = Audio::ConvertToDecibels(AbsAmplitude);
		const float MinDecibelNumericLimit = -160;
		FNumberFormattingOptions Formatting; 
		Formatting.SetMaximumFractionalDigits(1);
		return FText::Format(LOCTEXT("WaveformEditorValueGridLabel", "{0}"), (GridLineValue <= MinDecibelNumericLimit ? LOCTEXT("WaveformEditorValueGridNegativeInf", "-inf") : FText::AsNumber(GridLineValue, &Formatting)));
	};

	const UWaveformEditorWidgetsSettings* Settings = GetWaveformEditorWidgetsSettings();
	const bool bShowGrid = Settings ? Settings->ShowLoudnessGrid : true;
	const bool bShowLabels = Settings ? Settings->ShowLoudnessGridDecibelValues : true;
	const uint32 NumValueGridDivision = Settings ? Settings->MaxLoudnessGridDivisions : 3;


	const FSampledSequenceValueGridOverlayStyle& ValueGridStyle = WaveformEditorStyle->GetWidgetStyle<FSampledSequenceValueGridOverlayStyle>("WaveformEditorValueGrid.Style");
	
	ValueGridOverlay = SNew(SSampledSequenceValueGridOverlay)
		.DivideMode(SampledSequenceValueGridOverlay::EGridDivideMode::MidSplit)
		.ValueGridLabelGenerator(ValueGridDBConverter)
		.NumDimensions(DataView.NumDimensions)
		.MaxDivisionParameter(NumValueGridDivision)
		.HideLabels(!bShowLabels)
		.HideGrid(!bShowGrid)
		.Style(&ValueGridStyle);

	WaveformEditorStyle->OnNewValueGridOverlayStyle.AddSP(ValueGridOverlay.ToSharedRef(), &SSampledSequenceValueGridOverlay::OnStyleUpdated);
}

#undef LOCTEXT_NAMESPACE

