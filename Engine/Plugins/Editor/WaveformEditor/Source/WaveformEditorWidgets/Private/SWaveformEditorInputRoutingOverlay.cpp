// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWaveformEditorInputRoutingOverlay.h"

#include "AudioWidgetsUtils.h"
#include "Widgets/SLeafWidget.h"

void SWaveformEditorInputRoutingOverlay::Construct(const FArguments& InArgs, const TArray<TSharedPtr<SWidget>>& InOverlaidWidgets)
{
	if (InArgs._Style)
	{
		Style = InArgs._Style;
		DesiredWidth = Style->DesiredWidth;
		DesiredHeight = Style->DesiredHeight;
	}
	
	OverlaidWidgets = InOverlaidWidgets;
}

FReply SWaveformEditorInputRoutingOverlay::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return AudioWidgetsUtils::RouteMouseInput(&SWidget::OnMouseButtonDown, MouseEvent, MakeArrayView(OverlaidWidgets.GetData(), OverlaidWidgets.Num()));
}

FReply SWaveformEditorInputRoutingOverlay::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return AudioWidgetsUtils::RouteMouseInput(&SWidget::OnMouseButtonUp, MouseEvent, MakeArrayView(OverlaidWidgets.GetData(), OverlaidWidgets.Num()));
}

FReply SWaveformEditorInputRoutingOverlay::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return AudioWidgetsUtils::RouteMouseInput(&SWidget::OnMouseMove, MouseEvent, MakeArrayView(OverlaidWidgets.GetData(), OverlaidWidgets.Num()));
}

FReply SWaveformEditorInputRoutingOverlay::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (OnMouseWheelDelegate.IsBound())
	{
		return OnMouseWheelDelegate.Execute(MyGeometry, MouseEvent);
	}

	return AudioWidgetsUtils::RouteMouseInput(&SWidget::OnMouseWheel, MouseEvent, MakeArrayView(OverlaidWidgets.GetData(), OverlaidWidgets.Num()));
}

FCursorReply SWaveformEditorInputRoutingOverlay::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return AudioWidgetsUtils::RouteCursorQuery(CursorEvent, MakeArrayView(OverlaidWidgets.GetData(), OverlaidWidgets.Num()));
}

int32 SWaveformEditorInputRoutingOverlay::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	return ++LayerId;
}

FVector2D SWaveformEditorInputRoutingOverlay::ComputeDesiredSize(float) const
{
	return FVector2D(DesiredWidth, DesiredHeight);
}