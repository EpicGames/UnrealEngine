// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SLeafWidget.h"

class FWaveformEditorGridData;
class FSparseSampledSequenceTransportCoordinator;

class SWaveformEditorInputRoutingOverlay : public SLeafWidget
{
public:
	SLATE_BEGIN_ARGS(SWaveformEditorInputRoutingOverlay)
	{
	}

	SLATE_STYLE_ARGUMENT(FSampledSequenceViewerStyle, Style)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TArray<TSharedPtr<SWidget>>& InOverlaidWidgets);

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)override;
	virtual FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	FPointerEventHandler OnMouseWheelDelegate;

private:
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FVector2D ComputeDesiredSize(float) const override;

	TArray<TSharedPtr<SWidget>> OverlaidWidgets;

	const FSampledSequenceViewerStyle* Style = nullptr;

	float DesiredWidth = 1280.f;
	float DesiredHeight = 720.f;
};