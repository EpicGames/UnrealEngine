// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWaveformEditorMessageDialog.h"

#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "SWaveformEditorMessageDialog"

void SWaveformEditorMessageDialog::Construct(const FArguments& InArgs)
{
	ParentWindowPtr = InArgs._ParentWindow;
	FText DisplayMessage = InArgs._MessageToDisplay;

	ChildSlot
	[
		SNew(SBox)
		[
			SNew(SBorder)
			.Padding(0)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.AutoHeight()
				.Padding(16.0f, 16.0f, 16.0f, 0.0f)
				[
					SNew(STextBlock)
					.WrapTextAt(450.0f)
					.Text(DisplayMessage)
					.Justification(ETextJustify::Center)
				]
				+ SVerticalBox::Slot()
				.VAlign(VAlign_Bottom)
				.Padding(8.0f, 16.f, 8.0f, 16.0f)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					[
						SNew(SButton)
						.VAlign(VAlign_Center)
						.TextStyle(FAppStyle::Get(), "DialogButtonText")
						.Text(LOCTEXT("CloseWaveformEditorErrorMessageDialog", "Close"))
						.OnClicked(this, &SWaveformEditorMessageDialog::OnCloseButtonPressed)
						.IsEnabled(this, &SWaveformEditorMessageDialog::CanPressCloseButton)
					]
				]
			]
		]
	];
}


bool SWaveformEditorMessageDialog::CanPressCloseButton() const 
{
	return ParentWindowPtr.IsValid();
}

FReply SWaveformEditorMessageDialog::OnCloseButtonPressed() const
{
	if (ParentWindowPtr.IsValid())
	{
		ParentWindowPtr.Pin()->RequestDestroyWindow();
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE