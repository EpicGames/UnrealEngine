// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWaveformTransformationRenderLayer.h"

#include "Framework/Application/SlateApplication.h"
#include "Styling/AppStyle.h"

void SWaveformTransformationRenderLayer::PrivateRegisterAttributes(FSlateAttributeInitializer& AttributeInitializer)
{
}

void SWaveformTransformationRenderLayer::Construct(const FArguments& InArgs, TSharedRef<IWaveformTransformationRenderer> InTransformationRenderer)
{
	TransformationRenderer = InTransformationRenderer;
}

int32 SWaveformTransformationRenderLayer::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	return TransformationRenderer->OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

FVector2D SWaveformTransformationRenderLayer::ComputeDesiredSize(float) const
{
	return FVector2D(1280, 720);
}

FCursorReply SWaveformTransformationRenderLayer::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return TransformationRenderer->OnCursorQuery(MyGeometry, CursorEvent);
}

void SWaveformTransformationRenderLayer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	return TransformationRenderer->Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

FReply SWaveformTransformationRenderLayer::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return TransformationRenderer->OnMouseButtonDown(*this, MyGeometry, MouseEvent);
}

FReply SWaveformTransformationRenderLayer::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FSlateApplication::Get().SetKeyboardFocus(SharedThis(this));
	return TransformationRenderer->OnMouseButtonUp(*this, MyGeometry, MouseEvent);
}

FReply SWaveformTransformationRenderLayer::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return TransformationRenderer->OnMouseMove(*this, MyGeometry, MouseEvent);
}

FReply SWaveformTransformationRenderLayer::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return TransformationRenderer->OnMouseWheel(*this, MyGeometry, MouseEvent);
}

FReply SWaveformTransformationRenderLayer::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return TransformationRenderer->OnMouseButtonDoubleClick(*this, InMyGeometry, InMouseEvent);
}

void SWaveformTransformationRenderLayer::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return TransformationRenderer->OnMouseEnter(MyGeometry, MouseEvent);
}

void SWaveformTransformationRenderLayer::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	return TransformationRenderer->OnMouseLeave(MouseEvent);
}

FReply SWaveformTransformationRenderLayer::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return TransformationRenderer->OnKeyUp(MyGeometry, InKeyEvent);
}

