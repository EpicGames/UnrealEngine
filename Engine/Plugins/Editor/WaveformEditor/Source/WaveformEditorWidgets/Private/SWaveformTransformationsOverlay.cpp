// Copyright Epic Games, Inc. All Rights Reserved.

#include "SWaveformTransformationsOverlay.h"

#include "AudioWidgetsUtils.h"
#include "SWaveformTransformationRenderLayer.h"
#include "SparseSampledSequenceTransportCoordinator.h"
#include "WaveformEditorZoomController.h"
#include "Widgets/SOverlay.h"

void SWaveformTransformationsOverlay::Construct(const FArguments& InArgs, TArrayView<const FTransformationLayerRenderInfo> InTransformationRenderers, TSharedPtr<FWaveformEditorZoomController> InZoomController)
{
	TransformationRenderers = InTransformationRenderers;
	AnchorsRatioConverter = InArgs._AnchorsRatioConverter;

	check(InZoomController.IsValid());
	ZoomController = InZoomController;
	CreateLayout();
}

void SWaveformTransformationsOverlay::CreateLayout()
{
	const int32 NumLayers = TransformationRenderers.Num();
	TransformationLayers.Empty();
	TransformationLayers.SetNumZeroed(NumLayers);
	LayersSlots.Empty();
	LayersSlots.SetNumUninitialized(NumLayers);

	ChildSlot
	[
		SAssignNew(MainOverlayPtr, SOverlay)
	];

	for (int32 i = 0; i < NumLayers; ++i)
	{
		const FTransformationLayerRenderInfo& LayerRenderInfo = TransformationRenderers[i];
		
		if (LayerRenderInfo.Key)
		{
			float LeftAnchor = AnchorsRatioConverter(LayerRenderInfo.Value.Key);
			float RightAnchor = AnchorsRatioConverter(LayerRenderInfo.Value.Value);

			SConstraintCanvas::FSlot* SlotPtr = nullptr;

			TSharedPtr<SConstraintCanvas> ConstraintCanvasPtr = SNew(SConstraintCanvas)
				+ SConstraintCanvas::Slot()
				.Anchors(FAnchors(LeftAnchor, 0.f, RightAnchor, 1.f))
				.Expose(SlotPtr)
				[
					SAssignNew(TransformationLayers[i], SWaveformTransformationRenderLayer, LayerRenderInfo.Key.ToSharedRef())
				];

			MainOverlayPtr->AddSlot()
			[
				ConstraintCanvasPtr.ToSharedRef()
			];

			LayersSlots[i] = SlotPtr;

		}
		else
		{
			TransformationLayers[i] = nullptr;
			LayersSlots[i] = nullptr;
		}
	}
}

void SWaveformTransformationsOverlay::UpdateAnchors()
{
	check(TransformationRenderers.Num() == LayersSlots.Num())
	
	for (int32 i = 0; i < TransformationRenderers.Num(); ++i)
	{
		const FTransformationLayerRenderInfo& Layer = TransformationRenderers[i];

		if (Layer.Key)
		{
			float LeftAnchor = AnchorsRatioConverter(Layer.Value.Key);
			float RightAnchor = AnchorsRatioConverter(Layer.Value.Value);

			LayersSlots[i]->SetAnchors(FAnchors(LeftAnchor, 0.f, RightAnchor, 1.f));
		}
	}
}

void SWaveformTransformationsOverlay::OnLayerChainGenerated(FTransformationLayerRenderInfo* FirstLayerPtr, const int32 NLayers)
{
	TransformationRenderers = MakeArrayView(FirstLayerPtr, NLayers);
	CreateLayout();
}

void SWaveformTransformationsOverlay::UpdateLayerConstraints()
{
	UpdateAnchors();
}

FReply SWaveformTransformationsOverlay::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	FReply InputReply = AudioWidgetsUtils::RouteMouseInput(&SWidget::OnMouseButtonDown, MouseEvent, MakeArrayView(TransformationLayers.GetData(), TransformationLayers.Num()), false);

	// If the renderers do not consume input, then allow zoom scrolling
	if (!InputReply.IsEventHandled())
	{
		check(ZoomController != nullptr);
		InputReply = ZoomController->OnMouseButtonDown(MyGeometry, MouseEvent);
	}

	return InputReply;
}

FReply SWaveformTransformationsOverlay::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	ZoomController->OnMouseButtonUp(MyGeometry, MouseEvent);
	return AudioWidgetsUtils::RouteMouseInput(&SWidget::OnMouseButtonUp, MouseEvent, MakeArrayView(TransformationLayers.GetData(), TransformationLayers.Num()), false);
}

FReply SWaveformTransformationsOverlay::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	ZoomController->OnMouseMove(MyGeometry, MouseEvent);
	return AudioWidgetsUtils::RouteMouseInput(&SWidget::OnMouseMove, MouseEvent, MakeArrayView(TransformationLayers.GetData(), TransformationLayers.Num()), false);
}

FReply SWaveformTransformationsOverlay::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return AudioWidgetsUtils::RouteMouseInput(&SWidget::OnMouseWheel, MouseEvent, MakeArrayView(TransformationLayers.GetData(), TransformationLayers.Num()), false);
}

FCursorReply SWaveformTransformationsOverlay::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return AudioWidgetsUtils::RouteCursorQuery(CursorEvent, MakeArrayView(TransformationLayers.GetData(), TransformationLayers.Num()), false);
}
