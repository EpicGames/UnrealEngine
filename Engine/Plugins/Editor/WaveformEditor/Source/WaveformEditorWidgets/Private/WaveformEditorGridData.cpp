// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorGridData.h"

#include "Fonts/FontMeasure.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericPlatformMath.h"

FWaveformEditorGridData::FWaveformEditorGridData(const uint32 InTotalFrames, const uint32 InSampleRateHz, const float InGridSizePixels /*= 1.f*/, const FSlateFontInfo* InTicksTimeFont /*= nullptr*/)
	: TotalFrames(InTotalFrames)
	, DisplayRange(TRange<uint32>::Inclusive(0, InTotalFrames))
	, GridSizePixels(InGridSizePixels)
 	, TicksTimeFont(InTicksTimeFont)
 	, GridFrameRate(InSampleRateHz, 1)
{
	UpdateGridMetrics(GridSizePixels);
}

void FWaveformEditorGridData::UpdateDisplayRange(const TRange<uint32> InDisplayRange)
{
	check(DisplayRange.Size<uint32>() >= 2)
	DisplayRange = InDisplayRange;
	UpdateGridMetrics(GridSizePixels);
}

bool FWaveformEditorGridData::UpdateGridMetrics(const float InGridSizePixels)
{
	if (GridSizePixels != InGridSizePixels)
	{
		GridSizePixels = InGridSizePixels;
	}

	const float WaveformDurationSeconds = TotalFrames / (double) GridFrameRate.Numerator;
	const double StartTimeSeconds = DisplayRange.GetLowerBoundValue() / (double) GridFrameRate.Numerator;	
	const double DisplayedDuration = (DisplayRange.Size<double>() - 1) / GridFrameRate.Numerator; //we account for one less frame so we can distribute them evenly on the screen
	const double PixelsPerSecond = GridSizePixels / DisplayedDuration;
	double MajorGridStepSeconds = 0.0;
	float MinTicksPixelSpacing = 30.0f;
	
	if (TicksTimeFont)
	{
		TSharedRef<FSlateFontMeasure> FontMeasureService = FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
		FString LongestDisplayedTimeString = FString::Printf(TEXT("%.3f"), WaveformDurationSeconds * DisplayRange.GetUpperBoundValue());
		FVector2D MaxTextSize = FontMeasureService->Measure(LongestDisplayedTimeString, *TicksTimeFont);
		MinTicksPixelSpacing = MaxTextSize.X;
	}


	if (!GridFrameRate.ComputeGridSpacing(PixelsPerSecond, MajorGridStepSeconds, GridMetrics.NumMinorGridDivisions, MinTicksPixelSpacing + 5.f))
	{
		return false;
	}

	const bool bZeroGridOffset = DisplayRange.Size<uint32>() < GridSizePixels;
	const double GridOffset = bZeroGridOffset ? 0 : FGenericPlatformMath::Fmod(StartTimeSeconds, MajorGridStepSeconds);
	GridMetrics.FirstMajorTickX = (0.f - GridOffset) * PixelsPerSecond;
	GridMetrics.MajorGridXStep = MajorGridStepSeconds * PixelsPerSecond;
	GridMetrics.StartFrame = DisplayRange.GetLowerBoundValue();
	GridMetrics.PixelsPerFrame =  PixelsPerSecond / GridFrameRate.Numerator;
	GridMetrics.SampleRate = GridFrameRate.Numerator;

	if (OnGridMetricsUpdated.IsBound())
	{
		OnGridMetricsUpdated.Broadcast();
	}

	return true;
}

const FFixedSampledSequenceGridMetrics FWaveformEditorGridData::GetGridMetrics() const
{
	return GridMetrics;
}

void FWaveformEditorGridData::SetTicksTimeFont(const FSlateFontInfo* InNewFont)
{
	TicksTimeFont = InNewFont;
}

const float FWaveformEditorGridData::SnapPositionToClosestFrame(const float InPixelPosition) const
{
	const float DistanceFromPreviousSample = FGenericPlatformMath::Fmod(InPixelPosition, GridMetrics.PixelsPerFrame);
	const bool bSnapToNext = DistanceFromPreviousSample > GridMetrics.PixelsPerFrame / 2.f;
	float SnappedPosition = 0.f;

	if (bSnapToNext)
	{
		SnappedPosition = InPixelPosition + (GridMetrics.PixelsPerFrame - DistanceFromPreviousSample);
	}
	else
	{
		SnappedPosition = InPixelPosition -  DistanceFromPreviousSample;
	}


	return SnappedPosition;
}

const int64 FWaveformEditorGridData::GetFramePositionFromRatio(double Ratio) const
{
	const int64 DisplayedFrames = DisplayRange.GetUpperBoundValue() - DisplayRange.GetLowerBoundValue();

	const int64 FramePosition = FMath::RoundToInt64(Ratio * DisplayedFrames) + DisplayRange.GetLowerBoundValue();

	return FramePosition;
}

const double FWaveformEditorGridData::GetRatioFromFramePosition(int64 FramePosition) const
{
	if (!IsFramePositionInDisplayRange(FramePosition))
	{
		return -1; // Invalid ratio as we are now offscreen
	}

	const int64 PositionInDisplay = FramePosition - DisplayRange.GetLowerBoundValue();
	const double Range = DisplayRange.Size<double>();

	// A display range of 0 should not be possible as you would not be able to see anything in waveform editor
	check(Range > 0);

	return PositionInDisplay / Range;
}

const bool FWaveformEditorGridData::IsFramePositionInDisplayRange(int64 FramePosition) const
{
	return FramePosition >= DisplayRange.GetLowerBoundValue() && FramePosition <= DisplayRange.GetUpperBoundValue();
}
