// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorSequenceDataProvider.h"

#include "Audio.h"
#include "DSP/BufferVectorOperations.h"
#include "DSP/FloatArrayMath.h"
#include "IWaveformTransformationRenderer.h"
#include "Sound/SoundWave.h"
#include "WaveformTransformationDurationRenderer.h"

FWaveformEditorSequenceDataProvider::FWaveformEditorSequenceDataProvider(TObjectPtr<USoundWave> InSoundWave)
	: SoundWaveToRender(InSoundWave)
	, LayersFactory(MakeUnique<FWaveformTransformationRenderLayerFactory>())
{
	GenerateLayersChain();	

	if (!SoundWaveToRender->GetImportedSoundWaveData(ImportedRawPCMData, ImportedSampleRate, ImportedNumChannels))
	{
		UE_LOGF(LogAudio, Warning, "Failed to get transformations render data for: %ls", *SoundWaveToRender->GetPathName());
		return;
	}
}

FFixedSampledSequenceView FWaveformEditorSequenceDataProvider::RequestSequenceView(const TRange<double> DataRatioRange)
{
	check(DataRatioRange.GetLowerBoundValue() >= 0)
	check(DataRatioRange.GetUpperBoundValue() <= 1)
	check(DataRatioRange.GetLowerBoundValue() != DataRatioRange.GetUpperBoundValue())

	const int32 NumChannels = ImportedNumChannels;
	const uint8 MinFramesToDisplay = 2;
	const uint32 MinSamplesToDisplay = MinFramesToDisplay * NumChannels;
	const uint32 NumOriginalSamples = TransformedPCMData.Num();
	const uint32 NumOriginalFrames = NumOriginalSamples / NumChannels;

	const uint32 FirstRenderedSample = FMath::Clamp(FMath::RoundToInt32(NumOriginalFrames * DataRatioRange.GetLowerBoundValue()), 0, NumOriginalFrames - MinFramesToDisplay) * NumChannels;
	const uint32 NumFramesToRender = FMath::RoundToInt32(NumOriginalFrames * DataRatioRange.Size<double>());
	const uint32 NumSamplesToRender = FMath::Clamp(NumFramesToRender * NumChannels, MinSamplesToDisplay, NumOriginalSamples - FirstRenderedSample);

	check(NumSamplesToRender % NumChannels == 0 && FirstRenderedSample % NumChannels == 0);
;
	TArrayView<const float> SampleData = MakeArrayView(TransformedPCMData.GetData(), TransformedPCMData.Num()).Slice(FirstRenderedSample, NumSamplesToRender);

	FFixedSampledSequenceView DataView{ SampleData , NumChannels, ImportedSampleRate};
	OnDataViewGenerated.Broadcast(DataView, FirstRenderedSample);

	return DataView;
}

void FWaveformEditorSequenceDataProvider::GenerateLayersChain()
{
	check(SoundWaveToRender);

	if (SoundWaveToRender->Transformations.IsEmpty())
	{
		TransformationsToRender.Empty();
		RenderLayers.Empty();

		OnLayersChainGenerated.Broadcast(RenderLayers.GetData(), RenderLayers.Num());
	}

	TArray<TObjectPtr<UWaveformTransformationBase>> TempTransformationsToRender;
	TArray<FTransformationRenderLayerInfo> TempRenderLayers;

	for (const TObjectPtr<UWaveformTransformationBase>& Transformation : SoundWaveToRender->Transformations)
	{
		if (Transformation != nullptr)
		{
			TempTransformationsToRender.Add(Transformation);
			TSharedPtr<IWaveformTransformationRenderer> TransformationUI = LayersFactory->Create(Transformation);
			FTransformationRenderLayerInfo RenderLayerInfo = FTransformationRenderLayerInfo(TransformationUI, FTransformationLayerConstraints(0.f, 1.f));
			TempRenderLayers.Add(RenderLayerInfo);
		}
	}

	bool bTransformationIsSame = true;
	constexpr SIZE_T AccountForDurationLayer = 1;
	if (TempTransformationsToRender.Num() == TransformationsToRender.Num() && TempRenderLayers.Num() == RenderLayers.Num() - AccountForDurationLayer)
	{
		for (int32 Index = 0; Index < TempTransformationsToRender.Num(); ++Index)
		{
			if (TempTransformationsToRender[Index] != TransformationsToRender[Index])
			{
				bTransformationIsSame = false;
			}
		}

		if (bTransformationIsSame)
		{
			return;
		}
	}

	TransformationsToRender = MoveTemp(TempTransformationsToRender);
	RenderLayers = MoveTemp(TempRenderLayers);

	CreateDurationHighlightLayer();
	OnLayersChainGenerated.Broadcast(RenderLayers.GetData(), RenderLayers.Num());
}

void FWaveformEditorSequenceDataProvider::UpdateRenderElements()
{
	GenerateSequenceDataInternal();
	OnRenderElementsUpdated.Broadcast();
}

void FWaveformEditorSequenceDataProvider::GenerateSequenceDataInternal()
{
	check(SoundWaveToRender);
	check(ImportedNumChannels > 0);
	
	int64 NumWaveformSamples = ImportedRawPCMData.Num() * sizeof(uint8) / sizeof(int16);
	check(NumWaveformSamples >= ImportedNumChannels);

	NumOriginalWaveformFrames = NumWaveformSamples / ImportedNumChannels;
	int64 FirstEditedSample = 0;
	int64 LastEditedSample = NumWaveformSamples;

	if (TransformationsToRender.Num() > 0)
	{
		Audio::FWaveformTransformationWaveInfo TransformationInfo;

		// Changes in length based on transformations
		Audio::FAlignedFloatBuffer TransformationsBuffer;
		// Length remains unchanged 
		Audio::FAlignedFloatBuffer OutputBuffer;

		TransformationsBuffer.SetNumUninitialized(NumWaveformSamples);

		Audio::ArrayPcm16ToFloat(MakeArrayView((int16*)ImportedRawPCMData.GetData(), NumWaveformSamples), TransformationsBuffer);
		OutputBuffer = TransformationsBuffer;
		check(OutputBuffer.Num() >= ImportedNumChannels);

		TransformationInfo.Audio = &TransformationsBuffer;
		TransformationInfo.NumChannels = ImportedNumChannels;
		TransformationInfo.SampleRate = ImportedSampleRate;

		TArray<Audio::FTransformationPtr> Transformations = CreateTransformations();
		const bool bChainChangesFileLength = CanChainChangeFileLength(Transformations);

		for (int32 i = 0; i < Transformations.Num(); ++i)
		{
			Transformations[i]->ProcessAudio(TransformationInfo);

			if (RenderLayers[i].Key)
			{
				FWaveformTransformationRenderInfo RenderLayerInfo{ TransformationInfo.SampleRate, TransformationInfo.NumChannels, FirstEditedSample, NumWaveformSamples, LastEditedSample - FirstEditedSample };
				RenderLayers[i].Key->SetTransformationWaveInfo(MoveTemp(RenderLayerInfo));

				// For rendering, we want to be able to draw over all the samples, not just the ones available after processing
				RenderLayers[i].Value = FTransformationLayerConstraints(0, 1);
			}

			if (Transformations[i]->FileChangeLengthPriority() != Audio::ETransformationPriority::None)
			{
				FirstEditedSample += TransformationInfo.StartSampleOffset;
				
				check(FirstEditedSample <= OutputBuffer.Num() - ImportedNumChannels);
				check(FirstEditedSample >= 0);

				//Prevent a lower priority transformation from setting the LastEditedSample past the end of higher priority transformations
				if (Transformations[i]->FileChangeLengthPriority() == Audio::ETransformationPriority::High
					|| (Transformations[i]->FileChangeLengthPriority() != Audio::ETransformationPriority::None
						&& FirstEditedSample + TransformationsBuffer.Num() < LastEditedSample))
				{
					LastEditedSample = FirstEditedSample + TransformationsBuffer.Num();
				}
			}

			check(TransformationsBuffer.Num() >= ImportedNumChannels);
			check(TransformationsBuffer.Num() == (LastEditedSample - FirstEditedSample));
			check(TransformationsBuffer.Num() <= (OutputBuffer.Num() - FirstEditedSample));
			check(TransformationsBuffer.Num() > 0);
			check(FirstEditedSample >= 0);

			const int64 DestNum = static_cast<int64>(OutputBuffer.Num()) - FirstEditedSample;
			const SIZE_T DestSize = static_cast<uint64>(DestNum * sizeof(float));
			const SIZE_T CopySize = static_cast<uint64>(TransformationsBuffer.Num() * sizeof(float));
			check(DestSize >= CopySize);

			FMemory::Memcpy(&OutputBuffer.GetData()[FirstEditedSample], TransformationsBuffer.GetData(), CopySize);

			TransformationInfo.NumEditedSamples = 0;
			TransformationInfo.StartSampleOffset = 0;
		}

		check(DurationHiglightLayer)
		FWaveformTransformationRenderInfo DurationLayerInfo{ TransformationInfo.SampleRate, TransformationInfo.NumChannels, FirstEditedSample, LastEditedSample - FirstEditedSample };
		DurationHiglightLayer->SetTransformationWaveInfo(MoveTemp(DurationLayerInfo));
		DurationHiglightLayer->SetOriginalWaveformFrames(NumOriginalWaveformFrames);
		

		UpdateTransformedWaveformBounds(FirstEditedSample, LastEditedSample, NumWaveformSamples);

		if (!bChainChangesFileLength)
		{
			if (TransformedPCMData.Num() != TransformationsBuffer.Num())
			{
				TransformedPCMData.SetNumUninitialized(TransformationsBuffer.Num());
			}

			const SIZE_T CopySize = static_cast<uint64>(TransformationsBuffer.Num() * sizeof(float));
			FMemory::Memcpy(TransformedPCMData.GetData(), TransformationsBuffer.GetData(), CopySize);
		}
		else
		{
			if (TransformedPCMData.Num() != OutputBuffer.Num())
			{
				TransformedPCMData.SetNumUninitialized(OutputBuffer.Num());
			}

			const SIZE_T CopySize = static_cast<uint64>(OutputBuffer.Num() * sizeof(float));
			FMemory::Memcpy(TransformedPCMData.GetData(), OutputBuffer.GetData(), CopySize);
		}
		
		const float MaxValue = Audio::ArrayMaxAbsValue(TransformedPCMData);

		if (MaxValue > 1.f)
		{
			Audio::ArrayMultiplyByConstantInPlace(TransformedPCMData, 1.f / MaxValue);
		}
		
		check(TransformationInfo.NumChannels > 0);
		check(TransformationInfo.SampleRate > 0);
	}
	else
	{
		if (TransformedPCMData.Num() != NumWaveformSamples)
		{
			TransformedPCMData.SetNumUninitialized(NumWaveformSamples);

		}
		Audio::ArrayPcm16ToFloat(MakeArrayView((int16*)ImportedRawPCMData.GetData(), NumWaveformSamples), TransformedPCMData);
	}
}

void FWaveformEditorSequenceDataProvider::UpdateTransformedWaveformBounds(const uint32 FirstEditedSample, const uint32 LastEditedSample, const uint32 NumOriginalSamples)
{
	const double FirstEditedSampleOffset = FirstEditedSample / (double) NumOriginalSamples;
	const double LastEditedSampleOffset = (NumOriginalSamples - LastEditedSample) / (double) NumOriginalSamples;

	TransformedWaveformBounds.SetLowerBoundValue(FirstEditedSampleOffset);
	TransformedWaveformBounds.SetUpperBoundValue(1 - LastEditedSampleOffset);
}

TArray<Audio::FTransformationPtr> FWaveformEditorSequenceDataProvider::CreateTransformations() const
{
	TArray<Audio::FTransformationPtr> TransformationPtrs;

	for (const TObjectPtr<UWaveformTransformationBase>& TransformationBase : TransformationsToRender)
	{
		if(TransformationBase)
		{
			TransformationPtrs.Add(TransformationBase->CreateTransformation());
		}
	}

	return TransformationPtrs;
}

const bool FWaveformEditorSequenceDataProvider::CanChainChangeFileLength(const TArray<Audio::FTransformationPtr>& TransformationChain) const
{
	bool bCanChainChangeFileLength = false;

	for (const Audio::FTransformationPtr& Transformation : TransformationChain)
	{
		bCanChainChangeFileLength |= Transformation->FileChangeLengthPriority() != Audio::ETransformationPriority::None;
	}

	return bCanChainChangeFileLength;
}

TArrayView<const FTransformationRenderLayerInfo> FWaveformEditorSequenceDataProvider::GetTransformLayers() const
{
	return MakeArrayView(RenderLayers.GetData(), RenderLayers.Num());
}

const TRange<double> FWaveformEditorSequenceDataProvider::GetTransformedWaveformBounds() const
{
	return TransformedWaveformBounds;
}

void FWaveformEditorSequenceDataProvider::CreateDurationHighlightLayer()
{
	if (!DurationHiglightLayer)
	{
		DurationHiglightLayer = MakeShared<FWaveformTransformationDurationRenderer>(NumOriginalWaveformFrames);
	}

	DurationHiglightLayer->SetTransformationWaveInfo(FWaveformTransformationRenderInfo());
	RenderLayers.Emplace(DurationHiglightLayer, FTransformationLayerConstraints(0.f,1.f));
}
