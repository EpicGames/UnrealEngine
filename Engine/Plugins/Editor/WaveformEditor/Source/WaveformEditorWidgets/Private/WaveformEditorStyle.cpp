// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorStyle.h"

#include "Styling/SlateStyleRegistry.h"
#include "WaveformEditorWidgetsSettings.h"

static FLazyName PlayheadOverlayStyleName("WaveformEditorPlayheadOverlay.Style");
static FLazyName ValueGridOverlayStyleName("WaveformEditorValueGrid.Style");
static FLazyName WaveformEditorRulerStyleName("WaveformEditorRuler.Style");
static FLazyName WaveformViewerStyleName("WaveformViewer.Style");


FName FWaveformEditorStyle::StyleName("WaveformEditorStyle");
TUniquePtr<FWaveformEditorStyle> FWaveformEditorStyle::StyleInstance = nullptr;

FOnNewWaveformViewerStyle FWaveformEditorStyle::OnNewWaveformViewerStyle;
FOnNewPlayheadOverlayStyle FWaveformEditorStyle::OnNewPlayheadOverlayStyle;
FOnNewTimeRulerStyle FWaveformEditorStyle::OnNewTimeRulerStyle;
FOnNewValueGridOverlayStyle FWaveformEditorStyle::OnNewValueGridOverlayStyle;

FWaveformEditorStyle::FWaveformEditorStyle()
	: FSlateStyleSet(StyleName)
{
}

FWaveformEditorStyle::~FWaveformEditorStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FWaveformEditorStyle& FWaveformEditorStyle::Get()
{
	check(StyleInstance);
	return *StyleInstance;
}

void FWaveformEditorStyle::Init()
{
	if (StyleInstance == nullptr)
	{
		StyleInstance = MakeUnique<FWaveformEditorStyle>();
	}

	const UWaveformEditorWidgetsSettings* Settings = GetWidgetsSettings();
	
	check(Settings);
	Settings->OnSettingChanged().AddStatic(&FWaveformEditorStyle::OnWidgetSettingsUpdated);
	StyleInstance->SetParentStyleName("CoreStyle");

	//Waveform Viewer style
	FSampledSequenceViewerStyle WaveViewerStyle = CreateWaveformViewerStyleFromSettings(*Settings);
	StyleInstance->Set(WaveformViewerStyleName, WaveViewerStyle);

	//Playhead Overlay style
	FPlayheadOverlayStyle PlayheadOverlayStyle = CreatePlayheadOverlayStyleFromSettings(*Settings);
	StyleInstance->Set(PlayheadOverlayStyleName, PlayheadOverlayStyle);

	//Time Ruler style 
	FFixedSampleSequenceRulerStyle TimeRulerStyle = CreateTimeRulerStyleFromSettings(*Settings);
	StyleInstance->Set(WaveformEditorRulerStyleName, TimeRulerStyle);

	//ValueGridOverlayStyle
	FSampledSequenceValueGridOverlayStyle ValueGridOverlayStyle = CreateValueGridOverlayStyleFromSettings(*Settings);
	StyleInstance->Set(ValueGridOverlayStyleName, ValueGridOverlayStyle);

	FSlateStyleRegistry::RegisterSlateStyle(StyleInstance->Get());
}

const UWaveformEditorWidgetsSettings* FWaveformEditorStyle::GetWidgetsSettings() 
{
	const UWaveformEditorWidgetsSettings* WaveformEditorWidgetsSettings = GetDefault<UWaveformEditorWidgetsSettings>();
	check(WaveformEditorWidgetsSettings);

	return WaveformEditorWidgetsSettings;
}

void FWaveformEditorStyle::OnWidgetSettingsUpdated(const FName& PropertyName, const UWaveformEditorWidgetsSettings* Settings)
{
	check(Settings)

	if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, WaveformColor))
	{
		CreateWaveformViewerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, WaveformBackgroundColor))
	{
		CreateWaveformViewerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, WaveformLineThickness))
	{
		CreateWaveformViewerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, SampleMarkersSize))
	{
		CreateWaveformViewerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, MajorGridColor))
	{
		CreateWaveformViewerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, MinorGridColor))
	{
		CreateWaveformViewerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, ZeroCrossingLineThickness))
	{
		CreateWaveformViewerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, PlayheadColor))
	{
		CreatePlayheadOverlayStyleFromSettings(*Settings);
		CreateTimeRulerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, RulerBackgroundColor))
	{
		CreateTimeRulerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, RulerTicksColor))
	{
		CreateTimeRulerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, RulerTextColor))
	{
		CreateTimeRulerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, RulerFontSize))
	{
		CreateTimeRulerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, LoudnessGridColor))
	{
		CreateValueGridOverlayStyleFromSettings(*Settings);
		CreateWaveformViewerStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, LoudnessGridThickness))
	{
		CreateValueGridOverlayStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, LoudnessGridTextColor))
	{
		CreateValueGridOverlayStyleFromSettings(*Settings);
	}
	else if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformEditorWidgetsSettings, LoudnessGridTextSize))
	{
		CreateValueGridOverlayStyleFromSettings(*Settings);
	}

}

FSampledSequenceViewerStyle FWaveformEditorStyle::CreateWaveformViewerStyleFromSettings(const UWaveformEditorWidgetsSettings& InSettings)
{
	FSampledSequenceViewerStyle WaveViewerStyle = FSampledSequenceViewerStyle()
		.SetSequenceColor(InSettings.WaveformColor)
		.SetBackgroundColor(InSettings.WaveformBackgroundColor)
		.SetSequenceLineThickness(InSettings.WaveformLineThickness)
		.SetSampleMarkersSize(InSettings.SampleMarkersSize)
		.SetMajorGridLineColor(InSettings.MajorGridColor)
		.SetMinorGridLineColor(InSettings.MinorGridColor)
		.SetZeroCrossingLineColor(InSettings.LoudnessGridColor)
		.SetZeroCrossingLineThickness(InSettings.ZeroCrossingLineThickness);

	OnNewWaveformViewerStyle.Broadcast(WaveViewerStyle);

	return WaveViewerStyle;
}

FPlayheadOverlayStyle FWaveformEditorStyle::CreatePlayheadOverlayStyleFromSettings(const UWaveformEditorWidgetsSettings& InSettings)
{
	FPlayheadOverlayStyle PlayheadOverlayStyle = FPlayheadOverlayStyle().SetPlayheadColor(InSettings.PlayheadColor);

	OnNewPlayheadOverlayStyle.Broadcast(PlayheadOverlayStyle);

	return PlayheadOverlayStyle;
}

FFixedSampleSequenceRulerStyle FWaveformEditorStyle::CreateTimeRulerStyleFromSettings(const UWaveformEditorWidgetsSettings& InSettings)
{
	FFixedSampleSequenceRulerStyle TimeRulerStyle = FFixedSampleSequenceRulerStyle()
		.SetHandleColor(InSettings.PlayheadColor)
		.SetTicksColor(InSettings.RulerTicksColor)
		.SetTicksTextColor(InSettings.RulerTextColor)
		.SetHandleColor(InSettings.PlayheadColor)
		.SetFontSize(InSettings.RulerFontSize)
		.SetBackgroundColor(InSettings.RulerBackgroundColor);

	const ISlateStyle* AudioWidgetsStyle = FSlateStyleRegistry::FindSlateStyle("AudioWidgetsStyle");
	if (ensure(AudioWidgetsStyle))
	{
		TimeRulerStyle.SetHandleBrush(*AudioWidgetsStyle->GetBrush("SampledSequenceRuler.VanillaScrubHandleDown"));
	}

	OnNewTimeRulerStyle.Broadcast(TimeRulerStyle);

	return TimeRulerStyle;
}

FSampledSequenceValueGridOverlayStyle FWaveformEditorStyle::CreateValueGridOverlayStyleFromSettings(const UWaveformEditorWidgetsSettings& InSettings)
{
	FSampledSequenceValueGridOverlayStyle ValueGridOverlayStyle = FSampledSequenceValueGridOverlayStyle()
		.SetGridColor(InSettings.LoudnessGridColor)
		.SetGridThickness(InSettings.LoudnessGridThickness)
		.SetLabelTextColor(InSettings.LoudnessGridTextColor)
		.SetLabelTextFontSize(InSettings.LoudnessGridTextSize);

	OnNewValueGridOverlayStyle.Broadcast(ValueGridOverlayStyle);

	return ValueGridOverlayStyle;
}
