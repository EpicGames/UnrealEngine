// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorWidgetsModule.h"

#include "Modules/ModuleManager.h"
#include "WaveformEditorStyle.h"
#include "WaveformTransformationRendererMapper.h"

void FWaveformEditorWidgetsModule::StartupModule()
{
	// Initialize static style instance 
	FWaveformEditorStyle::Init();
	FWaveformTransformationRendererMapper::Init();
}

void FWaveformEditorWidgetsModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FWaveformEditorWidgetsModule, WaveformEditorWidgets)
