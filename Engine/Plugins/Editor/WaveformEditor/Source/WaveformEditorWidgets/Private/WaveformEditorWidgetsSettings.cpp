// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorWidgetsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveformEditorWidgetsSettings)

UWaveformEditorWidgetsSettings::UWaveformEditorWidgetsSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, PlayheadColor(FLinearColor(255.f, 0.1f, 0.2f, 1.f))
	, WaveformColor(FLinearColor::White)
	, WaveformLineThickness(1.f)
	, SampleMarkersSize(2.5f)
	, WaveformBackgroundColor(FLinearColor(0.02f, 0.02f, 0.02f, 1.f))
	, ZeroCrossingLineThickness(1.f)
	, MajorGridColor(FLinearColor::Black)
	, MinorGridColor(FLinearColor(0.f, 0.f, 0.f, 0.5f))
	, RulerBackgroundColor(FLinearColor::Black)
	, RulerTicksColor(FLinearColor(1.f, 1.f, 1.f, 0.9f))
	, RulerTextColor(FLinearColor(1.f, 1.f, 1.f, 0.9f))
	, RulerFontSize(10.f)
	, ShowLoudnessGrid(true)
	, ShowLoudnessGridDecibelValues(true)
	, MaxLoudnessGridDivisions(3)
	, LoudnessGridThickness(1.f)
	, LoudnessGridColor(FLinearColor::Black)
	, LoudnessGridTextColor(FLinearColor(1.f, 1.f, 1.f, 0.9f))
	, LoudnessGridTextSize(10.f)

{
}

FName UWaveformEditorWidgetsSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

FText UWaveformEditorWidgetsSettings::GetSectionText() const
{
	return NSLOCTEXT("WaveformEditorDisplay", "WaveformEditorDisplaySettingsSection", "Waveform Editor Display");
}

FName UWaveformEditorWidgetsSettings::GetSectionName() const
{
	return TEXT("Waveform Editor Display");
}

void UWaveformEditorWidgetsSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != nullptr)
	{
		SettingsChangedDelegate.Broadcast(PropertyChangedEvent.GetPropertyName(), this);
	}
}

FOnWaveformEditorWidgetsSettingsChanged& UWaveformEditorWidgetsSettings::OnSettingChanged()
{
	return SettingsChangedDelegate;
}

FOnWaveformEditorWidgetsSettingsChanged UWaveformEditorWidgetsSettings::SettingsChangedDelegate;
