// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "WaveformEditorWidgetsSettings.generated.h"

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnWaveformEditorWidgetsSettingsChanged, const FName& /*Property Name*/, const UWaveformEditorWidgetsSettings*);

UCLASS(config = EditorPerProjectUserSettings, defaultconfig, meta = (DisplayName = "Waveform Editor Display"))
class UWaveformEditorWidgetsSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	UWaveformEditorWidgetsSettings(const FObjectInitializer& ObjectInitializer);

	//~ Begin UDeveloperSettings interface
	virtual FName GetCategoryName() const;

#if WITH_EDITOR
	virtual FText GetSectionText() const override;

	virtual FName GetSectionName() const override;

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

#endif
	//~ End UDeveloperSettings interface

	static FOnWaveformEditorWidgetsSettingsChanged& OnSettingChanged();

	UPROPERTY(config, EditAnywhere, Category = "Playhead")
	FLinearColor PlayheadColor;

	UPROPERTY(config, EditAnywhere, Category = "Waveform Viewer")
	FLinearColor WaveformColor;

	UPROPERTY(config, EditAnywhere, Category = "Waveform Viewer", Meta = (ClampMin = "1", ClampMax = "10"))
	float WaveformLineThickness;

	UPROPERTY(config, EditAnywhere, Category = "Waveform Viewer", Meta = (ClampMin = "0", ClampMax = "30"))
	float SampleMarkersSize;

	UPROPERTY(config, EditAnywhere, Category = "Waveform Viewer")
	FLinearColor WaveformBackgroundColor;

	UPROPERTY(config, EditAnywhere, Category = "Waveform Viewer", Meta = (ClampMin = "1", ClampMax = "10"))
	float ZeroCrossingLineThickness;

	UPROPERTY(config, EditAnywhere, Category = "Waveform Viewer")
	FLinearColor MajorGridColor;
	
	UPROPERTY(config, EditAnywhere, Category = "Waveform Viewer")
	FLinearColor MinorGridColor;

	UPROPERTY(config, EditAnywhere, Category = "Ruler")
	FLinearColor RulerBackgroundColor;

	UPROPERTY(config, EditAnywhere, Category = "Ruler")
	FLinearColor RulerTicksColor;

	UPROPERTY(config, EditAnywhere, Category = "Ruler")
	FLinearColor RulerTextColor;

	UPROPERTY(config, EditAnywhere, Category = "Ruler", Meta = (ClampMin = "1", ClampMax = "15"))
	float RulerFontSize;

	UPROPERTY(config, EditAnywhere, Category = "Loudness Grid")
	bool ShowLoudnessGrid;

	UPROPERTY(config, EditAnywhere, Category = "Loudness Grid")
	bool ShowLoudnessGridDecibelValues;

	UPROPERTY(config, EditAnywhere, Category = "Loudness Grid", Meta = (ClampMin = "1", ClampMax = "10"))
	uint8 MaxLoudnessGridDivisions;

	UPROPERTY(config, EditAnywhere, Category = "Loudness Grid", Meta = (ClampMin = "1", ClampMax = "10"))
	float LoudnessGridThickness;

	UPROPERTY(config, EditAnywhere, Category = "Loudness Grid")
	FLinearColor LoudnessGridColor;

	UPROPERTY(config, EditAnywhere, Category = "Loudness Grid")
	FLinearColor LoudnessGridTextColor;

	UPROPERTY(config, EditAnywhere, Category = "Loudness Grid", Meta = (ClampMin = "1", ClampMax = "15"))
	float LoudnessGridTextSize;

	

private: 
	static FOnWaveformEditorWidgetsSettingsChanged SettingsChangedDelegate;
};