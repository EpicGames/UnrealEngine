// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformEditorZoomController.h"

#include "SparseSampledSequenceTransportCoordinator.h"
#include "WaveformEditorStyle.h"
#include <Blueprint/WidgetLayoutLibrary.h>

FWaveformEditorZoomController::FWaveformEditorZoomController(TSharedPtr<FSparseSampledSequenceTransportCoordinator> TransportCoordinator)
{
	check(TransportCoordinator);
	TransportController = TransportCoordinator;
}

void FWaveformEditorZoomController::ZoomIn()
{
	if (CanZoomIn())
	{
		ZoomLevel = FMath::Clamp(ZoomLevel += ZoomLevelStep, 0, LogRatioBase);
		ApplyZoom();
	}

}

bool FWaveformEditorZoomController::CanZoomIn() const
{
	return ZoomLevel + ZoomLevelStep <= LogRatioBase + ZoomLevelInitValue;
}

void FWaveformEditorZoomController::ZoomOut()
{
	if (CanZoomOut())
	{
		ZoomLevel = FMath::Clamp(ZoomLevel -= ZoomLevelStep, 0, LogRatioBase);
		ApplyZoom();
	}
}

bool FWaveformEditorZoomController::CanZoomOut() const
{
	return ZoomLevel - ZoomLevelStep >= 0;
}


void FWaveformEditorZoomController::ZoomByDelta(const float Delta)
{
	if (Delta >= 0.f)
	{
		ZoomIn();
	}
	else
	{
		ZoomOut();
	}
}

float FWaveformEditorZoomController::GetZoomRatio() const
{
	return 1 - ConvertZoomLevelToLogRatio();
}

void FWaveformEditorZoomController::CheckBounds(const FGeometry& AllottedGeometry)
{
	if (bIsScrolling)
	{
		FVector2D MouseAbsolutePosition = UWidgetLayoutLibrary::GetMousePositionOnPlatform();

		if (!AllottedGeometry.IsUnderLocation(MouseAbsolutePosition))
		{
			bIsScrolling = false;

			check(TransportController);
			TransportController->SetScrolling(false);
		}
	}
}

FReply FWaveformEditorZoomController::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		const FVector2D LocalCursorPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		LocalCursorXPosition = FMath::Clamp(LocalCursorPosition.X, 0.f, MyGeometry.GetLocalSize().X);

		bIsScrolling = true;

		check(TransportController);
		TransportController->SetScrolling(true);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply FWaveformEditorZoomController::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		bIsScrolling = false;

		check(TransportController);
		TransportController->SetScrolling(false);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply FWaveformEditorZoomController::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (bIsScrolling)
	{
		check(TransportController);
		check(MyGeometry.GetLocalSize().X > 0);

		const FVector2D LocalCursorPosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
		const float LocalCursorXClamped = FMath::Clamp(LocalCursorPosition.X, 0.f, MyGeometry.GetLocalSize().X);
		const float MovementDifference = (LocalCursorXClamped - LocalCursorXPosition);
		const float MoveRatio = MovementDifference / MyGeometry.GetLocalSize().X;

		TransportController->MoveWaveformView(-MoveRatio);

		LocalCursorXPosition = LocalCursorXClamped;
	}

	return FReply::Unhandled();
}

void FWaveformEditorZoomController::ApplyZoom()
{
	OnZoomRatioChanged.Broadcast(1 - ConvertZoomLevelToLogRatio());
}

float FWaveformEditorZoomController::ConvertZoomLevelToLogRatio() const
{
	return FMath::Clamp(FMath::LogX(LogRatioBase, ZoomLevel), 0.f, 1.f);
}
