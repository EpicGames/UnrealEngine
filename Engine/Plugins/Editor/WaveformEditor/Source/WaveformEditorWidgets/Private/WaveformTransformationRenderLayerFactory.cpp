// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationRenderLayerFactory.h"

#include "IWaveformTransformation.h"
#include "IWaveformTransformationRenderer.h"
#include "WaveformTransformationRendererMapper.h"

TSharedPtr<IWaveformTransformationRenderer> FWaveformTransformationRenderLayerFactory::Create(TObjectPtr<UWaveformTransformationBase> InTransformationToRender)
{
	TSharedPtr<IWaveformTransformationRenderer> OutRenderer = nullptr;
	UClass* TransformationClass = InTransformationToRender->GetClass();

	FWaveformTransformRendererInstantiator* Instantiator = FWaveformTransformationRendererMapper::Get().GetRenderer(InTransformationToRender->GetClass());

	if (Instantiator)
	{
		OutRenderer = (*Instantiator)();

		if (OutRenderer)
		{
			OutRenderer->SetWaveformTransformation(InTransformationToRender);
		}
	}

	return OutRenderer;
}

TSharedPtr<IWaveformTransformationRenderer> FWaveformTransformationRenderLayerFactory::Create(TObjectPtr<UWaveformTransformationBase> InTransformationToRender, TArray<TSharedRef<IPropertyHandle>>& TransformationsProperties)
{
	TSharedPtr<IWaveformTransformationRenderer> OutRenderer = Create(InTransformationToRender);

	if (OutRenderer)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OutRenderer->SetPropertyHandles(TransformationsProperties);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	return OutRenderer;
}