// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "HAL/Platform.h"
#include "Widgets/SCompoundWidget.h"
#include "IFixedSampledSequenceViewReceiver.h"

#define UE_API WAVEFORMEDITORWIDGETS_API

enum class ESampledSequenceDisplayUnit;
struct FNotifyingAudioWidgetStyle;

class FSparseSampledSequenceTransportCoordinator;
class FWaveformEditorGridData;
class FWaveformEditorRenderData;
class FWaveformEditorStyle;
class FWaveformEditorZoomController;
class SBorder;
class SFixedSampledSequenceRuler;
class SFixedSampledSequenceViewer;
class SPlayheadOverlay;
class SSampledSequenceValueGridOverlay;
class SWaveformEditorInputRoutingOverlay;
class SWaveformTransformationsOverlay;
class UWaveformEditorWidgetsSettings;

class STransformedWaveformViewPanel : public SCompoundWidget, public IFixedSampledSequenceViewReceiver 
{
public: 
	SLATE_BEGIN_ARGS(STransformedWaveformViewPanel) {}
		
		SLATE_DEFAULT_SLOT(FArguments, InArgs)
		
		SLATE_ARGUMENT(TSharedPtr<SWaveformTransformationsOverlay>, TransformationsOverlay)

		SLATE_ARGUMENT(TSharedPtr<FWaveformEditorZoomController>, ZoomController)

		SLATE_ARGUMENT(FPointerEventHandler, OnPlayheadOverlayMouseButtonUp)

		SLATE_ARGUMENT(FPointerEventHandler, OnTimeRulerMouseButtonUp)

		SLATE_ARGUMENT(FPointerEventHandler, OnTimeRulerMouseButtonDown)

		SLATE_ARGUMENT(FPointerEventHandler, OnTimeRulerMouseMove)

		SLATE_ARGUMENT(FPointerEventHandler, OnMouseWheel)

	SLATE_END_ARGS()
	
	UE_API void Construct(const FArguments& InArgs, const FFixedSampledSequenceView& InView);
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API virtual void ReceiveSequenceView(const FFixedSampledSequenceView InView, const uint32 FirstSampleIndex = 0) override;
	UE_API void SetPlayheadRatio(const double InRatio);
	UE_API void SetPlayheadFramePosition(const int64 FramePosition);
	UE_API void ResolveFromFramePosition();

	UE_API void SetOnPlayheadOverlayMouseButtonUp(FPointerEventHandler InEventHandler);
	UE_API void SetOnTimeRulerMouseButtonUp(FPointerEventHandler InEventHandler);
	UE_API void SetOnTimeRulerMouseButtonDown(FPointerEventHandler InEventHandler);
	UE_API void SetOnTimeRulerMouseMove(FPointerEventHandler InEventHandler);
	UE_API void SetOnMouseWheel(FPointerEventHandler InEventHandler);

	UE_API FReply LaunchTimeRulerContextMenu();

	const int64 GetPlayheadFramePosition() { return CachedPlayheadFramePosition; }

private:
	UE_API void CreateLayout();

	UE_API void SetUpGridData();
	UE_API void SetupPlayheadOverlay();
	UE_API void SetUpTimeRuler(TSharedRef<FWaveformEditorGridData> InGridData);
	UE_API void SetUpWaveformViewer(TSharedRef<FWaveformEditorGridData> InGridData, const FFixedSampledSequenceView& InView);
	UE_API void SetUpInputRoutingOverlay();
	UE_API void SetUpInputOverrides(const FArguments& InArgs);
	UE_API void SetUpValueGridOverlay();
	UE_API void SetUpBackground();


	UE_API void UpdateDisplayUnit(const ESampledSequenceDisplayUnit InDisplayUnit);
	UE_API void UpdatePlayheadPosition(const float PaintedWidth);
	UE_API void UpdateBackground(const FSampledSequenceViewerStyle UpdatedStyle);

	UE_API void OnWaveEditorWidgetSettingsUpdated(const FName& PropertyName, const UWaveformEditorWidgetsSettings* Settings);

	TSharedPtr<FWaveformEditorGridData> GridData;

	TSharedPtr<SFixedSampledSequenceRuler> TimeRuler;
	TSharedPtr<SFixedSampledSequenceViewer> WaveformViewer;
 	TSharedPtr<SWaveformTransformationsOverlay> WaveformTransformationsOverlay;
	TSharedPtr<SWaveformEditorInputRoutingOverlay> InputRoutingOverlay;
	TSharedPtr<SPlayheadOverlay> PlayheadOverlay;
	TSharedPtr<SSampledSequenceValueGridOverlay> ValueGridOverlay;
	TSharedPtr<SBorder> BackgroundBorder;
	TSharedPtr<FWaveformEditorZoomController> ZoomController;

	float CachedPixelWidth = 0.f;

	ESampledSequenceDisplayUnit DisplayUnit;

	FWaveformEditorStyle* WaveformEditorStyle;

	FFixedSampledSequenceView DataView;
	
	double CachedPlayheadRatio = 0.0;
	int64 CachedPlayheadFramePosition = 0;
	UE_API const UWaveformEditorWidgetsSettings* GetWaveformEditorWidgetsSettings();
};

#undef UE_API
