// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#define UE_API WAVEFORMEDITORWIDGETS_API

class SWindow;

class SWaveformEditorMessageDialog : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SWaveformEditorMessageDialog) {}

	/** A reference to the parent window */
	SLATE_ARGUMENT(TSharedPtr<SWindow>, ParentWindow)

	/** The Message To Display */
	SLATE_ARGUMENT(FText, MessageToDisplay)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs);

private: 
	/** The parent window of this widget */
	TWeakPtr<SWindow> ParentWindowPtr;

	UE_API bool CanPressCloseButton() const;
	
	UE_API FReply OnCloseButtonPressed() const;
};

#undef UE_API
