// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "IWaveformTransformationRenderer.h"

#define UE_API WAVEFORMEDITORWIDGETS_API

class SWaveformTransformationRenderLayer : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SWaveformTransformationRenderLayer, SCompoundWidget)

public:
	SLATE_BEGIN_ARGS(SWaveformTransformationRenderLayer) {}
		SLATE_DEFAULT_SLOT(FArguments, InArgs)
	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TSharedRef<IWaveformTransformationRenderer> InTransformationRenderer);
	
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	UE_API FVector2D ComputeDesiredSize(float) const override;
	UE_API FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	UE_API FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	UE_API FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual bool SupportsKeyboardFocus() const override { return true; }

private:
	TSharedPtr<IWaveformTransformationRenderer> TransformationRenderer;
};

#undef UE_API
