// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaveformEditorZoomController.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Layout/SConstraintCanvas.h"

#define UE_API WAVEFORMEDITORWIDGETS_API

class FSparseSampledSequenceTransportCoordinator;
class IWaveformTransformationRenderer;
class SOverlay;
class SWaveformTransformationRenderLayer;

using FTransformationLayerRenderInfo = TPair<TSharedPtr<IWaveformTransformationRenderer>, TPair<float,float>>;

class SWaveformTransformationsOverlay : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SWaveformTransformationsOverlay) 
		: _AnchorsRatioConverter([](const float InRatio) {return InRatio; })
	{
	}

		SLATE_DEFAULT_SLOT(FArguments, InArgs)

		SLATE_ARGUMENT(TFunction<float(const float)>, AnchorsRatioConverter)

	SLATE_END_ARGS()

	UE_API void Construct(const FArguments& InArgs, TArrayView< const FTransformationLayerRenderInfo> InTransformationRenderers, TSharedPtr<FWaveformEditorZoomController> InZoomController);
	UE_API void OnLayerChainGenerated(FTransformationLayerRenderInfo* FirstLayerPtr, const int32 NLayers);
	UE_API void UpdateLayerConstraints();

	UE_API FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API FReply OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	
private:
	UE_API void CreateLayout();
	UE_API void UpdateAnchors();
	
	TSharedPtr<SOverlay> MainOverlayPtr;
	TArray<TSharedPtr<SWidget>> TransformationLayers;
	TArray<SConstraintCanvas::FSlot*> LayersSlots;
	TArrayView<const FTransformationLayerRenderInfo> TransformationRenderers;
	TFunction<float(const float)> AnchorsRatioConverter;
	TSharedPtr<FWaveformEditorZoomController> ZoomController;
};

#undef UE_API
