// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "IFixedSampledSequenceGridService.h"
#include "Math/Range.h"
#include "Misc/FrameRate.h"

#define UE_API WAVEFORMEDITORWIDGETS_API

DECLARE_MULTICAST_DELEGATE(FOnWaveformEditorGridUpdated)

struct FSlateFontInfo;

class FWaveformEditorGridData : public IFixedSampledSequenceGridService
{
public: 
	UE_API explicit FWaveformEditorGridData(const uint32 InTotalFrames, const uint32 InSampleRateHz, const float InGridSizePixels = 1.f, const FSlateFontInfo* InTicksTimeFont = nullptr);

	UE_API void UpdateDisplayRange(const TRange<uint32> InDisplayRange);
	UE_API bool UpdateGridMetrics(const float InGridSizePixels);
	UE_API virtual const FFixedSampledSequenceGridMetrics GetGridMetrics() const override;
	UE_API void SetTicksTimeFont(const FSlateFontInfo* InNewFont);
	UE_API const float SnapPositionToClosestFrame(const float InPixelPosition) const;
	UE_API const int64 GetFramePositionFromRatio(double Ratio) const;
	UE_API const double GetRatioFromFramePosition(int64 FramePosition) const;
	UE_API const bool IsFramePositionInDisplayRange(int64 FramePosition) const;

	const TRange<uint32> GetDisplayRange() { return DisplayRange; }

	FOnWaveformEditorGridUpdated OnGridMetricsUpdated;

private:
	FFixedSampledSequenceGridMetrics GridMetrics;
	uint32 TotalFrames = 0;
	TRange<uint32> DisplayRange;

	float GridSizePixels = 0.f;
	const FSlateFontInfo* TicksTimeFont = nullptr;
	FFrameRate GridFrameRate;
};

#undef UE_API
