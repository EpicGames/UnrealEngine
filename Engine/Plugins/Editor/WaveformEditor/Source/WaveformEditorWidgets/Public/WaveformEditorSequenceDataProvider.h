// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "IFixedSampledSequenceViewProvider.h"
#include "IWaveformTransformation.h"
#include "WaveformTransformationRenderLayerFactory.h"

#define UE_API WAVEFORMEDITORWIDGETS_API

class IWaveformTransformationRenderer;
class FWaveformTransformationDurationRenderer;
class USoundWave;
class UWaveformTransformationBase;
struct FFixedSampledSequenceViewView;

using FTransformationLayerConstraints = TPair<float, float>;
using FTransformationRenderLayerInfo = TPair<TSharedPtr<IWaveformTransformationRenderer>, FTransformationLayerConstraints>;
using FTransformationToPropertiesPair = TPair<TObjectPtr<UWaveformTransformationBase>, TArray<TSharedRef<IPropertyHandle>>>;
using FTransformationsToPropertiesArray = TArray<FTransformationToPropertiesPair>;

DECLARE_DELEGATE_OneParam(FOnTransformationsPropertiesRequired, FTransformationsToPropertiesArray& /*Transformations Properties To Hand*/)
DECLARE_MULTICAST_DELEGATE(FOnRenderElementsUpdated)
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnDataViewGenerated, FFixedSampledSequenceView, const uint32 /*FirstSampleIndex*/)
DECLARE_MULTICAST_DELEGATE_TwoParams(FOnLayersChainGenerated, FTransformationRenderLayerInfo* /*First Layer ptr*/, const int32 /* NLayers */)



// FWaveformEditorSequenceDataProvider													
//																						
// The Waveform Editor Sequence data provider can produce UI to display a chain of		
// waveform transformations.															
// 																						
// The main UI elements created are:													
//  TransformedPCMData : a float array containing the transformed samples of the		
//						 waveform, that can be requested through the					
//					     IFixedSampledSequenceViewProvider interface.					
//  Transform Layers : an array of UI renderers for each transformation					
//                     in the chain. Renderers should be contained in dummy widgets.	
//																						
// These are created so that the entire stack of transformation is displayable.			
// E.g: if a 10s long file is trimmed from second 2, seconds 0 to 1 will 				
// still be present in the render data.													
// UI widgets are passed a struct with information about the transformation to display	
// property (e.g. StartFrameOffset, SampleDuration, etc.).								
// 																						
// UIs for different transformations are registered with and spawned by					
// FWaveformTransformationRenderLayerFactory.											
// Transformations don't necessarily have a widget UI. In that case, they are only 		
// reflected in the render data. 	


class FWaveformEditorSequenceDataProvider : public IFixedSampledSequenceViewProvider, public TSharedFromThis<FWaveformEditorSequenceDataProvider>
{
public:

	UE_API FWaveformEditorSequenceDataProvider(TObjectPtr<USoundWave> InSoundWave);

	UE_API virtual FFixedSampledSequenceView RequestSequenceView(const TRange<double> DataRatioRange) override;

	/** Used to generate the stack of transformations UI					*/
	/** Should be called when the waveform transformation chain is changed	*/
	UE_API void GenerateLayersChain();

	/** Used to generate updated sequence data and pass transformation info to the widgets */
	/** Should be called when the transformations parameters are changed				 */
	UE_API void UpdateRenderElements();

	UE_API TArrayView<const FTransformationRenderLayerInfo> GetTransformLayers() const;

	UE_API const TRange<double> GetTransformedWaveformBounds() const;

	const TArray<float>& GetTransformedPCMData() const { return TransformedPCMData; };

	/** Called when a new layer chain of transformations UI is created */
	FOnLayersChainGenerated OnLayersChainGenerated;

	/** Called when the different render elements are updated */
	FOnRenderElementsUpdated OnRenderElementsUpdated;

	/** Called when new data view is generated */
	FOnDataViewGenerated OnDataViewGenerated;

private:
	UE_API void GenerateSequenceDataInternal();

	UE_API void UpdateTransformedWaveformBounds(const uint32 FirstEditedSample, const uint32 LastEditedSample, const uint32 NumOriginalSamples);

	UE_API void CreateDurationHighlightLayer();

	UE_API TArray<Audio::FTransformationPtr> CreateTransformations() const;
	UE_API const bool CanChainChangeFileLength(const TArray<Audio::FTransformationPtr>& TransformationArray) const;

	TArray<TObjectPtr<UWaveformTransformationBase>> TransformationsToRender;
	TArray<FTransformationRenderLayerInfo> RenderLayers;

	TObjectPtr<USoundWave> SoundWaveToRender = nullptr;

	/** Imported Soundwave Data **/
	TArray<uint8> ImportedRawPCMData;
	uint16 ImportedNumChannels;
	uint32 ImportedSampleRate;

	TArray<float> TransformedPCMData;

	TUniquePtr<FWaveformTransformationRenderLayerFactory> LayersFactory = nullptr;
	TSharedPtr<FWaveformTransformationDurationRenderer> DurationHiglightLayer = nullptr;

	uint32 NumOriginalWaveformFrames = 0;

	/* The bounds of the transformed waveform in relation to the original */
	TRange<double> TransformedWaveformBounds = TRange<double>::Inclusive(0, 1);
};

#undef UE_API
