// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "Delegates/Delegate.h"
#include "Styling/SlateStyle.h"

class UWaveformEditorWidgetsSettings;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewPlayheadOverlayStyle, FPlayheadOverlayStyle)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewTimeRulerStyle, FFixedSampleSequenceRulerStyle)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewValueGridOverlayStyle, FSampledSequenceValueGridOverlayStyle)
DECLARE_MULTICAST_DELEGATE_OneParam(FOnNewWaveformViewerStyle, FSampledSequenceViewerStyle)

class FWaveformEditorStyle
	: public FSlateStyleSet

{
public:
	FWaveformEditorStyle();
	~FWaveformEditorStyle();

	static FName StyleName;

	/** Access the singleton instance for this style set */
	static FWaveformEditorStyle& Get();
	static void Init();

	static FOnNewPlayheadOverlayStyle OnNewPlayheadOverlayStyle;
	static FOnNewTimeRulerStyle OnNewTimeRulerStyle;
	static FOnNewValueGridOverlayStyle OnNewValueGridOverlayStyle;
	static FOnNewWaveformViewerStyle OnNewWaveformViewerStyle;

private:
	static TUniquePtr<FWaveformEditorStyle> StyleInstance;

	static const UWaveformEditorWidgetsSettings* GetWidgetsSettings();
	static void  OnWidgetSettingsUpdated(const FName& PropertyName, const UWaveformEditorWidgetsSettings* Settings);

	static FFixedSampleSequenceRulerStyle CreateTimeRulerStyleFromSettings(const UWaveformEditorWidgetsSettings& InSettings);
	static FPlayheadOverlayStyle CreatePlayheadOverlayStyleFromSettings(const UWaveformEditorWidgetsSettings& InSettings);
	static FSampledSequenceValueGridOverlayStyle CreateValueGridOverlayStyleFromSettings(const UWaveformEditorWidgetsSettings& InSettings);
	static FSampledSequenceViewerStyle CreateWaveformViewerStyleFromSettings(const UWaveformEditorWidgetsSettings& InSettings);
};