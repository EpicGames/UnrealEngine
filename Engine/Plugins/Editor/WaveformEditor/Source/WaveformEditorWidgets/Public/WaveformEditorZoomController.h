// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AudioWidgetsSlateTypes.h"
#include "Delegates/Delegate.h"
#include "SparseSampledSequenceTransportCoordinator.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Widgets/SLeafWidget.h"

#define UE_API WAVEFORMEDITORWIDGETS_API

DECLARE_MULTICAST_DELEGATE_OneParam(FOnZoomRatioChanged, const float /* New Zoom Ratio */);

class FWaveformEditorZoomController
{
public:
	SLATE_BEGIN_ARGS(FWaveformEditorZoomController)
		{
		}

	SLATE_END_ARGS()
	UE_API FWaveformEditorZoomController(TSharedPtr<FSparseSampledSequenceTransportCoordinator> TransportCoordinator);

	UE_API void ZoomIn();
	UE_API bool CanZoomIn() const;
	UE_API void ZoomOut();
	UE_API bool CanZoomOut() const;
	UE_API void ZoomByDelta(const float Delta);

	UE_API float GetZoomRatio() const;

	UE_API void CheckBounds(const FGeometry& AllottedGeometry);
	UE_API FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UE_API FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	UE_API FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);

	FOnZoomRatioChanged OnZoomRatioChanged;

private:
	UE_API void ApplyZoom();
	UE_API float ConvertZoomLevelToLogRatio() const;

	float ZoomLevelInitValue = 1.f;
	float ZoomLevel = ZoomLevelInitValue;
	float ZoomLevelStep = 2.f;
	const uint32 LogRatioBase = 100;

	bool bIsScrolling = false;
	float LocalCursorXPosition = 0.0f;

	TSharedPtr<FSparseSampledSequenceTransportCoordinator> TransportController = nullptr;
};

#undef UE_API
