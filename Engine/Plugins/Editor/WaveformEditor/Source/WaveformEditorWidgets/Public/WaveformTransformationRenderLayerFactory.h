// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectPtr.h"
#include "Templates/SharedPointer.h"

#define UE_API WAVEFORMEDITORWIDGETS_API

class IWaveformTransformationRenderer;
class UWaveformTransformationBase;
class IPropertyHandle;

class FWaveformTransformationRenderLayerFactory
{
public:
	~FWaveformTransformationRenderLayerFactory() = default;

	UE_API TSharedPtr<IWaveformTransformationRenderer> Create(TObjectPtr<UWaveformTransformationBase> InTransformationToRender);

	UE_DEPRECATED(5.7, "TransformationsProperties in Create has been deprecated.")
	UE_API TSharedPtr<IWaveformTransformationRenderer> Create(TObjectPtr<UWaveformTransformationBase> InTransformationToRender, TArray<TSharedRef<IPropertyHandle>>& TransformationsProperties);
};

#undef UE_API
