// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

namespace UnrealBuildTool.Rules
{
	public class WaveformEditorWidgets : ModuleRules
	{
		public WaveformEditorWidgets(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"AudioExtensions",
					"AudioWidgets",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"EditorStyle",
					"Engine",
					"InputCore",
					"SignalProcessing",
					"Slate",
					"SlateCore", 
					"ToolMenus",
					"UMG",
					"WaveformTransformationsWidgets",
				}
			);
		}
	}
}