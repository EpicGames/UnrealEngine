// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformAudioAnalysisFunctions.h"

#include "LoudnessFactory.h"
#include "Algo/MaxElement.h"
#include "LKFSNRTFactory.h"
#include "MeterFactory.h"
#include <limits>

namespace WaveformAudioAnalysis
{
	constexpr float DbMin = -std::numeric_limits<float>::infinity();

	// GatedLoudness starts analysis with a -70dB gate, if something falls below the gated value, we should return DbMin
	constexpr float NoiseFloor = -70.f;

	float GetRMSPeak(const Audio::FAlignedFloatBuffer& InputAudio, float SampleRate, int32 NumChannels)
	{
		check(NumChannels > 0);
		check(SampleRate > 0.f);

		const float AudioLengthSeconds = InputAudio.Num() / NumChannels / SampleRate;
		const float RMSAnalysisPeriod = 0.3f;
		const bool bAnalyzeAsSingleBuffer = RMSAnalysisPeriod > AudioLengthSeconds;

		Audio::FMeterSettings Settings;
		Settings.AnalysisPeriod = bAnalyzeAsSingleBuffer ? AudioLengthSeconds : RMSAnalysisPeriod;

		if (bAnalyzeAsSingleBuffer)
		{
			Audio::FMeterAnalyzer Analyzer(SampleRate, NumChannels, Settings);
			Audio::FMeterAnalyzerResults Results = Analyzer.ProcessAudio(InputAudio);
			const float* MeterValue = Algo::MaxElement(Results.MeterValues, [](const float A, const float B)
				{
					return A < B;
				});

			if (MeterValue)
			{
				if (*MeterValue < NoiseFloor)
				{
					return DbMin;
				}

				return *MeterValue;
			}
		}
		else
		{
			Audio::FMeterFactory Analyzer;
			TUniquePtr<Audio::IAnalyzerResult> Result = Analyzer.NewResult();
			TUniquePtr<Audio::IAnalyzerWorker> AnalyzerWorker = Analyzer.NewWorker({ static_cast<int32>(SampleRate), NumChannels }, &Settings);

			if (AnalyzerWorker == nullptr)
			{
				return DbMin;
			}

			AnalyzerWorker->Analyze(MakeArrayView(InputAudio), Result.Get());

			Audio::FMeterResult* MeterResult = static_cast<Audio::FMeterResult*>(Result.Get());

			if (MeterResult)
			{
				const TArray<Audio::FMeterEntry>& LoudnessArray = MeterResult->GetMeterArray();
				auto MeterEntry = Algo::MaxElement(LoudnessArray, [](const Audio::FMeterEntry& A, const Audio::FMeterEntry& B)
					{
						return A.MeterValue < B.MeterValue;
					});

				if (MeterEntry)
				{
					if (MeterEntry->MeterValue < NoiseFloor)
					{
						return DbMin;
					}

					return MeterEntry->MeterValue;
				}
			}
		}

		return DbMin;
	}

	float GetLoudnessPeak(const Audio::FAlignedFloatBuffer& InputAudio, float SampleRate, int32 NumChannels)
	{
		Audio::FLoudnessFactory Analyzer;
		Audio::FLoudnessSettings Settings;
		Settings.AnalysisPeriod = 1.f;
		
		// Could implement bAnalyzeAsSingleBuffer optimization here as above
		
		TUniquePtr<Audio::IAnalyzerResult> Result = Analyzer.NewResult();
		TUniquePtr<Audio::IAnalyzerWorker> AnalyzerWorker = Analyzer.NewWorker({ static_cast<int32>(SampleRate), NumChannels }, &Settings);

		if (AnalyzerWorker == nullptr)
		{
			return DbMin;
		}

		AnalyzerWorker->Analyze(MakeArrayView(InputAudio), Result.Get());

		Audio::FLoudnessResult* LoudnessResult = static_cast<Audio::FLoudnessResult*>(Result.Get());

		if (LoudnessResult)
		{
			const TArray<Audio::FLoudnessEntry>& LoudnessArray = LoudnessResult->GetLoudnessArray();
			auto LoudestEntry = Algo::MaxElement(LoudnessArray, [](const Audio::FLoudnessEntry& A, const Audio::FLoudnessEntry& B)
				{
					return A.Loudness < B.Loudness;
				});

			if (LoudestEntry)
			{
				if (LoudestEntry->Loudness < NoiseFloor)
				{
					return DbMin;
				}

				return LoudestEntry->Loudness;
			}
		}

		return DbMin;
	}

	float GetPeakSampleValue(const Audio::FAlignedFloatBuffer& InputAudio)
	{
		const float MaxValue = Audio::ArrayMaxAbsValue(InputAudio);
		float DbValue = Audio::ConvertToDecibels(MaxValue);

		if (DbValue < NoiseFloor)
		{
			DbValue = DbMin;
		}

		return DbValue;
	}

	float GetLUFS(const Audio::FAlignedFloatBuffer& InputAudio, float SampleRate, int32 NumChannels)
	{
		Audio::FLKFSNRTFactory Factory;
		Audio::FAnalyzerNRTParameters AnalyzerParameters(SampleRate, NumChannels);
		Audio::FLKFSNRTSettings LKFSSettings;

		TUniquePtr<Audio::IAnalyzerNRTResult> Result = Factory.NewResult();
		TUniquePtr<Audio::IAnalyzerNRTWorker> Worker = Factory.NewWorker(AnalyzerParameters, &LKFSSettings);

		if (Worker == nullptr)
		{
			return DbMin;
		}

		Worker->Analyze(MakeArrayView(InputAudio), Result.Get());
		Worker->Finalize(Result.Get());

		Audio::FLKFSNRTResult* LKFSResult = static_cast<Audio::FLKFSNRTResult*>(Result.Get());

		if (LKFSResult)
		{
			const float GatedLoudness = LKFSResult->FindAggregateLoudnessStats(LKFSResult->ChannelIndexOverall)->GatedLoudness;

			if (GatedLoudness == FLKFSNRTAggregateStats::InvalidLoudness || GatedLoudness < NoiseFloor)
			{
				// Sample not large enough to analyze
				return DbMin;
			}

			return GatedLoudness;
		}

		return DbMin;
	}
}
