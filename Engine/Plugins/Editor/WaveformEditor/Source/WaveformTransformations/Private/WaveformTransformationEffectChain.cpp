// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationEffectChain.h"
#include "Engine/Engine.h"
#include "Sound/SoundEffectPreset.h"
#include "Sound/SoundEffectSource.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveformTransformationEffectChain)

FWaveTransformationEffectChain::FWaveTransformationEffectChain(TArray<TObjectPtr<USoundEffectSourcePreset>>& InEffectPresets)
{
	for(USoundEffectSourcePreset* Preset : InEffectPresets)
	{
		if(Preset)
		{
			Presets.Add(TStrongObjectPtr<USoundEffectSourcePreset>(Preset));
		}
	}
}

void FWaveTransformationEffectChain::ProcessAudio(Audio::FWaveformTransformationWaveInfo& InOutWaveInfo) const
{
	check(InOutWaveInfo.Audio != nullptr);
	check(InOutWaveInfo.Audio->Num() % InOutWaveInfo.NumChannels == 0);

	uint32 AudioDeviceId = GEngine->GetMainAudioDeviceID();

	FSoundEffectSourceInitData InitData;
	InitData.SampleRate = InOutWaveInfo.SampleRate;
	InitData.AudioClock = 0.0;
	InitData.NumSourceChannels = InOutWaveInfo.NumChannels;
	InitData.AudioDeviceId = AudioDeviceId;

	TArray<TSharedPtr<class FSoundEffectSource>> Effects;

	for(const TStrongObjectPtr<USoundEffectSourcePreset>& Preset : Presets)
	{
		InitData.ParentPresetUniqueId = Preset->GetUniqueID();

		TSoundEffectSourcePtr Effect = USoundEffectPreset::CreateInstance<FSoundEffectSourceInitData, FSoundEffectSource>(InitData, *Preset);
		Effect->SetEnabled(true);
		Effects.Add(Effect);
	}
	
	Audio::FAlignedFloatBuffer& InputAudio = *InOutWaveInfo.Audio;

	const int32 NumBlockSamples = 1024 * InOutWaveInfo.NumChannels;
	
	Audio::FAlignedFloatBuffer OutputBuffer;

	OutputBuffer.AddUninitialized(NumBlockSamples);

	FSoundEffectSourceInputData InputData;
	InputData.AudioClock = 0.0;
	InputData.CurrentPitch = 1.f;
	InputData.CurrentVolume = 1.f;
	InputData.NumSamples = NumBlockSamples;
	InputData.SpatParams = FSpatializationParams();
	InputData.CurrentPlayFraction = 0.f;
	InputData.InputSourceEffectBufferPtr = InputAudio.GetData();

	for(TSoundEffectSourcePtr& EffectPtr : Effects)
	{
		EffectPtr->Update();
		
		int32 SampleIndex = 0;
		int32 SamplesToProcess = 0;

		while(SampleIndex < InputAudio.Num())
		{
			InputData.InputSourceEffectBufferPtr = &InputAudio[SampleIndex];
			SamplesToProcess = FMath::Clamp(InputAudio.Num() - SampleIndex, 0, NumBlockSamples);

			EffectPtr->ProcessAudio(InputData, OutputBuffer.GetData());
			
			FMemory::Memcpy(InputData.InputSourceEffectBufferPtr, OutputBuffer.GetData(), SamplesToProcess * sizeof(float));
			SampleIndex += SamplesToProcess;
		}
	}
}

Audio::FTransformationPtr UWaveformTransformationEffectChain::CreateTransformation() const
{
	TArray<TObjectPtr<USoundEffectSourcePreset>> Effects;
	if(EffectChain)
	{
		for(FSourceEffectChainEntry Entry : EffectChain->Chain)
		{
			if(Entry.bBypass == false && Entry.Preset != nullptr)
			{
				Effects.Add(Entry.Preset);
			}
		}
	}
	else
	{
		for(USoundEffectSourcePreset* Preset : InlineEffects)
		{
			if(Preset)
			{
				Effects.Add(Preset);
			}
		}
	}
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return MakeUnique<FWaveTransformationEffectChain>(Effects);
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}
