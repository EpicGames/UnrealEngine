// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationFade.h"

#include "AudioCompressionSettingsUtils.h"
#include "Editor.h"
#include "WaveformTransformationLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveformTransformationFade)

namespace WaveformTransformationFadePrivate
{
	// Makes the two functions for SCurve meet at (0.5, 0.5)
	const double XMultiplier = FMath::Sqrt(10.0) / 5.0;

	constexpr float Linear = 1.f;

	static void ApplyFadeIn(Audio::FAlignedFloatBuffer& InputAudio, const int64 AudioSizeInSamples, const FTransformationFadeFunctionData& FadeFunctions, const int32 NumChannels, const float SampleRate, const int64 StartFrameOffset)
	{
		if (FadeFunctions.FadeIn == nullptr)
		{
			return;
		}

		check(NumChannels > 0);
		check(SampleRate > 0.f);

		// Assert against buffer overflows
		ensure(StartFrameOffset >= 0);
		ensure(StartFrameOffset == 0 || (INT64_MAX / StartFrameOffset) >= NumChannels);
		const int64 StartSampleOffset = static_cast<int64>(StartFrameOffset * NumChannels);
		ensure(INT64_MAX - AudioSizeInSamples >= StartSampleOffset);

		const int64 NumSamples = (AudioSizeInSamples + StartSampleOffset);
		const int64 NumFrames = NumSamples / NumChannels;

		const int64 StartFrame = FadeFunctions.FadeIn->FrameOffset - StartFrameOffset;

		if (StartFrame < 0)
		{
			return;
		}

		const float FadeDuration = FadeFunctions.FadeIn->Duration;
		const int64 NumFramesAfterOffset = NumFrames - StartFrame;
		const int64 FadeNumFrames = FMath::Min(FMath::FloorToInt64(FadeDuration * SampleRate), NumFramesAfterOffset);

		if (NumFrames <= 1 || FadeDuration < SMALL_NUMBER || FadeNumFrames == 0 || (StartFrame + FadeNumFrames) * NumChannels > InputAudio.Num())
		{
			return;
		}

		float* InputPtr = InputAudio.GetData() + StartFrame * NumChannels;

		for (int64 FrameIndex = 0; FrameIndex < FadeNumFrames; ++FrameIndex)
		{
			const float FadeFraction = static_cast<float>(FrameIndex) / static_cast<float>(FadeNumFrames);
			check(FadeFraction <= 1.0f);
			check(FadeFraction >= 0.0f);

			const float EnvValue = FadeFunctions.FadeIn->GetFadeInCurveValue(FadeFraction);

			for (int64 ChannelIt = 0; ChannelIt < NumChannels; ++ChannelIt)
			{
				*(InputPtr + ChannelIt) *= EnvValue;
			}

			InputPtr += NumChannels;
		}
	}

	static void ApplyFadeOut(Audio::FAlignedFloatBuffer& InputAudio, const int64 AudioSizeInSamples, const FTransformationFadeFunctionData& FadeFunctions, const int32 NumChannels, const float SampleRate, const int64 EndFrameOffset)
	{
		if (FadeFunctions.FadeOut == nullptr)
		{
			return;
		}

		check(NumChannels > 0);
		check(SampleRate > 0.f);

		// Assert against buffer overflows
		ensure(EndFrameOffset == 0 || (INT64_MAX / EndFrameOffset) >= NumChannels);
		const int64 EndSampleOffset = static_cast<int64>(EndFrameOffset * NumChannels);
		ensure(INT64_MAX - AudioSizeInSamples >= EndSampleOffset);

		const int64 NumSamples = (AudioSizeInSamples + EndSampleOffset);
		const int64 NumFramesWithEndFrameOffset = NumSamples / NumChannels;

		const int64 FrameOffset = FadeFunctions.FadeOut->FrameOffset;

		if (FrameOffset < 0)
		{
			return;
		}

		const float FadeDuration = FadeFunctions.FadeOut->Duration;
		const int64 NumFramesAfterOffset = NumFramesWithEndFrameOffset - FrameOffset;
		const int64 FadeNumFrames = FMath::Min(FMath::FloorToInt64(FadeDuration * SampleRate), NumFramesAfterOffset);

		if (NumFramesAfterOffset <= 1 || FadeDuration < SMALL_NUMBER || FadeNumFrames == 0)
		{
			return;
		}

		const int64 FadeNumFramesFromEnd = FMath::Min(FadeNumFrames + FrameOffset, NumFramesWithEndFrameOffset);
		const int64 StartSampleIndex = NumSamples - (static_cast<int64>(FadeNumFramesFromEnd) * NumChannels);
		check(StartSampleIndex >= 0);
		check(AudioSizeInSamples <= InputAudio.Num());

		if (StartSampleIndex + (static_cast<int64>(FadeNumFrames) * NumChannels) > AudioSizeInSamples)
		{
			return;
		}

		float* InputPtr = &InputAudio[StartSampleIndex];

		for (int64 FrameIndex = 0; FrameIndex < FadeNumFrames; ++FrameIndex)
		{
			const float FadeFraction = static_cast<float>(FrameIndex) / static_cast<float>(FadeNumFrames);
			check(FadeFraction <= 1.0f);
			check(FadeFraction >= 0.0f);

			const float EnvValue = FadeFunctions.FadeOut->GetFadeOutCurveValue(FadeFraction);

			for (int64 ChannelIt = 0; ChannelIt < NumChannels; ++ChannelIt)
			{
				*(InputPtr + ChannelIt) *= EnvValue;
			}

			InputPtr += NumChannels;
		}
	}
}

const TMap<EWaveEditorTransformationFadeMode, TSubclassOf<UTransformationFadeFunction>> UWaveformTransformationFade::FadeModeToFadeFunctionMap =
{
	{EWaveEditorTransformationFadeMode::Linear, UTransformationFadeFunctionLinear::StaticClass()},
	{EWaveEditorTransformationFadeMode::Exponential, UTransformationFadeCurveFunctionExponential::StaticClass()},
	{EWaveEditorTransformationFadeMode::Logarithmic, UTransformationFadeCurveFunctionLogarithmic::StaticClass()},
	{EWaveEditorTransformationFadeMode::Sigmoid, UTransformationFadeCurveFunctionSigmoid::StaticClass()}
};

FWaveTransformationFade::FWaveTransformationFade(const TArray<FTransformationFadeFunctionData>& InFadeRegions, const int64 InStartFrameOffset, const int64 InEndFrameOffset)
	: FadeRegions(InFadeRegions)
	, StartFrameOffset(InStartFrameOffset)
	, EndFrameOffset(InEndFrameOffset) {}

void FWaveTransformationFade::ProcessAudio(Audio::FWaveformTransformationWaveInfo& InOutWaveInfo) const
{
	check(InOutWaveInfo.SampleRate > 0.f && InOutWaveInfo.Audio != nullptr);
	check(InOutWaveInfo.NumChannels > 0);

	Audio::FAlignedFloatBuffer& InputAudio = *InOutWaveInfo.Audio;

	// If InputAudio is smaller than the size of a frame, do not process the audio.
	const int64 InputAudioNum = static_cast<int64>(InputAudio.Num());
	check(InputAudioNum >= InOutWaveInfo.NumChannels);

	InOutWaveInfo.NumEditedSamples = InputAudioNum;

	for (const FTransformationFadeFunctionData& Region : FadeRegions)
	{
		const bool bProcessFadeIn = Region.FadeIn && Region.FadeIn->Duration > 0.f;
		const bool bProcessFadeOut = Region.FadeOut && Region.FadeOut->Duration > 0.f;

		if (bProcessFadeIn)
		{
			WaveformTransformationFadePrivate::ApplyFadeIn(InputAudio, InputAudioNum, Region, InOutWaveInfo.NumChannels, InOutWaveInfo.SampleRate, StartFrameOffset);
		}

		if (bProcessFadeOut)
		{
			WaveformTransformationFadePrivate::ApplyFadeOut(InputAudio, InputAudioNum, Region, InOutWaveInfo.NumChannels, InOutWaveInfo.SampleRate, EndFrameOffset);
		}
	}
}

UWaveformTransformationFade::UWaveformTransformationFade(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	FTransformationFadeFunctionData FadeRegion;
	
	FadeRegion.FadeIn = CreateDefaultSubobject<UTransformationFadeCurveFunctionExponential>(TEXT("FadeInFunction"));
	FadeRegion.FadeIn->Duration = 0.f;
	FadeRegion.FadeOut = CreateDefaultSubobject<UTransformationFadeCurveFunctionExponential>(TEXT("FadeOutFunction"));
	FadeRegion.FadeOut->Duration = 0.f;
	
	FadeRegions.Add(FadeRegion);
}

#if WITH_EDITOR
void UWaveformTransformationFade::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (AvailableWaveformDuration >= 0.f)
	{
		// if AvailableWaveformDuration >= 0, we know SampleRate has been initialized
		check(SampleRate > 0.f);
		const float SoundDuration = OriginalNumFrames / SampleRate;

		// Clamp frame offset to be within the original waveform 
		const int64 MaxFrameOffset = OriginalNumFrames;

		for (FTransformationFadeFunctionData& FadeRegion : FadeRegions)
		{			
			if (FadeRegion.FadeIn)
			{
				FadeRegion.FadeIn->FrameOffset = FMath::Clamp(FadeRegion.FadeIn->FrameOffset, 0, MaxFrameOffset);

				const float MaxFadeInDuration = SoundDuration - (FadeRegion.FadeIn->FrameOffset / SampleRate);
				FadeRegion.FadeIn->Duration = FMath::Clamp(FadeRegion.FadeIn->Duration, 0.f, MaxFadeInDuration);
			}
			else
			{
				FadeRegion.FadeIn = NewObject<UTransformationFadeCurveFunctionExponential>(this, UTransformationFadeCurveFunctionExponential::StaticClass(), NAME_None, RF_Transactional);
				FadeRegion.FadeIn->Duration = 0.f;
			}

			if (FadeRegion.FadeOut)
			{
				FadeRegion.FadeOut->FrameOffset = FMath::Clamp(FadeRegion.FadeOut->FrameOffset, 0, MaxFrameOffset);

				const float MaxFadeOutDuration = SoundDuration - (FadeRegion.FadeOut->FrameOffset / SampleRate);;

				if (FadeRegion.bLinkDurations)
				{
					FadeRegion.FadeOut->Duration = FMath::Clamp(FadeRegion.FadeIn->Duration, 0, MaxFadeOutDuration);
				}
				else
				{
					FadeRegion.FadeOut->Duration = FMath::Clamp(FadeRegion.FadeOut->Duration, 0.f, MaxFadeOutDuration);
				}
			}
			else
			{
				FadeRegion.FadeOut = NewObject<UTransformationFadeCurveFunctionExponential>(this, UTransformationFadeCurveFunctionExponential::StaticClass(), NAME_None, RF_Transactional);
				FadeRegion.FadeOut->Duration = 0.f;
			}
		}
	}

	// Clamp values first, before triggering the transformation update in UWaveformTransformationBase::PostEditChangeProperty(PropertyChangedEvent)
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

FString UWaveformTransformationFade::GetTransformationHash() const
{
	using FPCU = FPlatformCompressionUtilities;
	FString Hash;

	float FadeInDuration = 0.f;
	int64 FadeInFrameOffset = 0;
	float FadeInCurve = WaveformTransformationFadePrivate::Linear;

	float FadeOutDuration = 0.f;
	int64 FadeOutFrameOffset = 0;
	float FadeOutCurve = WaveformTransformationFadePrivate::Linear;

	if (FadeRegions.Num() > 0)
	{
		FPCU::AppendHash(Hash, TEXT("FRC"), FadeRegions.Num());
	}

	for (const FTransformationFadeFunctionData& FadeRegion : FadeRegions)
	{
		FPCU::AppendHash(Hash, TEXT("FRLD"), FadeRegion.bLinkDurations);

		if (FadeRegion.FadeIn)
		{
			FadeInDuration = FadeRegion.FadeIn->Duration;
			FadeInFrameOffset = FadeRegion.FadeIn->FrameOffset;
			if (FadeRegion.FadeIn.IsA<UTransformationFadeCurveFunction>())
			{
				TObjectPtr<UTransformationFadeCurveFunction> CurveFunction = CastChecked<UTransformationFadeCurveFunction>(FadeRegion.FadeIn);
				FadeInCurve = CurveFunction->GetFadeCurve();
			}
			else
			{
				FadeInCurve = WaveformTransformationFadePrivate::Linear;
			}

			FPCU::AppendHash(Hash, TEXT("FIC"), FadeInCurve);
			FPCU::AppendHash(Hash, TEXT("FID"), FadeInDuration);
			FPCU::AppendHash(Hash, TEXT("FIFO"), FadeInFrameOffset);
		}

		if (FadeRegion.FadeOut)
		{
			FadeOutDuration = FadeRegion.FadeOut->Duration;
			FadeOutFrameOffset = FadeRegion.FadeOut->FrameOffset;
			if (FadeRegion.FadeOut.IsA<UTransformationFadeCurveFunction>())
			{
				TObjectPtr<UTransformationFadeCurveFunction> CurveFunction = CastChecked<UTransformationFadeCurveFunction>(FadeRegion.FadeOut);
				FadeOutCurve = CurveFunction->GetFadeCurve();
			}
			else
			{
				FadeOutCurve = WaveformTransformationFadePrivate::Linear;
			}

			FPCU::AppendHash(Hash, TEXT("FOC"), FadeOutCurve);
			FPCU::AppendHash(Hash, TEXT("FOD"), FadeOutDuration);
			FPCU::AppendHash(Hash, TEXT("FOFO"), FadeOutFrameOffset);
		}
	}
	
	return Hash;
}
#endif // WITH_EDITOR

Audio::FTransformationPtr UWaveformTransformationFade::CreateTransformation() const
{
	return MakeUnique<FWaveTransformationFade>(FadeRegions, StartFrameOffset, EndFrameOffset);
}

void UWaveformTransformationFade::UpdateConfiguration(FWaveTransformUObjectConfiguration& InOutConfiguration)
{
	UpdateDurationProperties(InOutConfiguration.EndTime - InOutConfiguration.StartTime);
	SampleRate = InOutConfiguration.SampleRate;
	StartFrameOffset = InOutConfiguration.GetStartFrameOffset();
	EndFrameOffset = InOutConfiguration.GetEndFrameOffset();
	OriginalNumFrames = InOutConfiguration.GetOriginalNumFrames();
}

void UWaveformTransformationFade::LimitFadeRegionsToOne()
{
	if (FadeRegions.Num() > 1)
	{
		FadeRegions.SetNum(1, EAllowShrinking::Yes);
		UE_LOGF(LogWaveformTransformation, Warning, "Fade Transformation had more than one fade region. REMOVING excess fade regions!");
	}
}

void UWaveformTransformationFade::UpdateDurationProperties(const float InAvailableDuration)
{
	check(InAvailableDuration >= 0.f);
	AvailableWaveformDuration = InAvailableDuration;
}

const double UTransformationFadeFunctionLinear::GetFadeInCurveValue(const double FadeFraction) const
{
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return FadeFraction;
}

const double UTransformationFadeFunctionLinear::GetFadeOutCurveValue(const double FadeFraction) const
{
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return 1.0 - FadeFraction;
}

const double UTransformationFadeCurveFunctionExponential::GetFadeInCurveValue(const double FadeFraction) const
{
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return FMath::Pow(FadeFraction, FadeCurve);
}

const double UTransformationFadeCurveFunctionExponential::GetFadeOutCurveValue(const double FadeFraction) const
{
	// Pow functions return NaN for negative bases with non-integer exponents
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return FMath::Pow(-FadeFraction + 1.0, FadeCurve);
}

const double UTransformationFadeCurveFunctionLogarithmic::GetFadeInCurveValue(const double FadeFraction) const
{
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return FMath::Pow(FadeFraction, FadeCurve);
}

const double UTransformationFadeCurveFunctionLogarithmic::GetFadeOutCurveValue(const double FadeFraction) const
{
	// Pow functions return NaN for negative bases with non-integer exponents
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return FMath::Pow(-FadeFraction + 1.0, FadeCurve);
}

const double UTransformationFadeCurveFunctionSigmoid::GetFadeInCurveValue(const double FadeFraction) const
{
	// LogX will produce NaN if FadeFraction is <= 0 or is 1
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);
	check(WaveformTransformationFadePrivate::XMultiplier > 0.0);

	if (FadeFraction == 0.f || FadeFraction == 1.f)
	{
		return FadeFraction;
	}

	double CurveValue = 1;
	const double x = FadeFraction;
	const double a = static_cast<float>(-FMath::Clamp(SigmoidFadeCurve, 0.1f, 1.0f));
	const double k = 20 * FMath::Pow(a, 5);

	if (FadeFraction <= 0.5)
	{
		CurveValue = -(FMath::Tanh(k * PI * FMath::LogX(10, WaveformTransformationFadePrivate::XMultiplier * x) + ((k * PI) / 2))) / 2 + 0.5;
	}
	else
	{
		CurveValue = (FMath::Tanh(k * PI * FMath::LogX(10, WaveformTransformationFadePrivate::XMultiplier * (-x + 1)) + ((k * PI) / 2))) / 2 + 0.5;
	}

	return CurveValue;
}

const double UTransformationFadeCurveFunctionSigmoid::GetFadeOutCurveValue(const double FadeFraction) const
{
	// LogX will produce NaN if FadeFraction is <= 0 or is 1
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);
	check(WaveformTransformationFadePrivate::XMultiplier > 0.0);

	if (FadeFraction == 0.f || FadeFraction == 1.f)
	{
		return 1.0 - FadeFraction;
	}

	double CurveValue = 1;
	const double x = FadeFraction;
	const double a = static_cast<float>(FMath::Clamp(SigmoidFadeCurve, 0.1f, 1.0f));
	const double k = 20 * FMath::Pow(a, 5);

	if (FadeFraction <= 0.5)
	{
		CurveValue = -(FMath::Tanh(k * PI * FMath::LogX(10, WaveformTransformationFadePrivate::XMultiplier * x) + ((k * PI) / 2))) / 2 + 0.5;
	}
	else
	{
		CurveValue = (FMath::Tanh(k * PI * FMath::LogX(10, WaveformTransformationFadePrivate::XMultiplier * (-x + 1)) + ((k * PI) / 2))) / 2 + 0.5;
	}

	return CurveValue;
}
