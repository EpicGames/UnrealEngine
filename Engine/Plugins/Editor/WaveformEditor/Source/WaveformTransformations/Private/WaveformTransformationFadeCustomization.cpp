// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationFadeCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "WaveformTransformationFade.h"

TSharedRef<IPropertyTypeCustomization> FWaveformTransformationFadeCustomization::MakeInstance()
{
	return MakeShareable(new FWaveformTransformationFadeCustomization);
}

void FWaveformTransformationFadeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle,
	FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	constexpr bool bDisplayDefaultPropertyButtons = false;

	HeaderRow.NameContent()[PropertyHandle->CreatePropertyNameWidget()]
	.ValueContent()[PropertyHandle->CreatePropertyValueWidget(bDisplayDefaultPropertyButtons)];
}

void FWaveformTransformationFadeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle,
	IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> FadeRegionHandle = PropertyHandle->GetChildHandle(UWaveformTransformationFade::GetFadeRegionsPropertyName());

	if (FadeRegionHandle && FadeRegionHandle->IsValidHandle())
	{
		ChildBuilder.AddProperty(FadeRegionHandle.ToSharedRef());
	}
}