// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationMarkers.h"

#include "AudioCompressionSettingsUtils.h"
#include "Editor.h"
#include "Sound/SoundWave.h"
#include "WaveformTransformationLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveformTransformationMarkers)

#define LOCTEXT_NAMESPACE "WaveformTransformationMarkers"

namespace
{
	static const TArray<int32> SizeIncrements = { 1, 10, 50, 100 };
	static const TArray<FKey> IndexKeyCodes = { EKeys::One, EKeys::Two, EKeys::Three, EKeys::Four, EKeys::Five, EKeys::Six, EKeys::Seven, EKeys::Eight, EKeys::Nine };

	// Create a copy of the CuePoint data that is shifted by the StartFrameOffset created by transformations like TrimFade
	const TArray<FSoundWaveCuePoint> GetAdjustedCuePoints(const TArray<FSoundWaveCuePoint>& InCuePoints, const int64 InStartFrameOffset, bool bRemoveTrimmed = false)
	{
		check(InStartFrameOffset >= 0);
		if (InStartFrameOffset == 0)
		{
			return InCuePoints;
		}

		TArray<FSoundWaveCuePoint> FilteredAndAdjustedCuePoints;
		FilteredAndAdjustedCuePoints.Reserve(InCuePoints.Num());
		
		for (const FSoundWaveCuePoint& Marker : InCuePoints)
		{
			const int64 NewPosition = Marker.FramePosition - InStartFrameOffset;

			if (NewPosition < 0 && bRemoveTrimmed)
			{
				// Keep loop regions even if only the end frame is within the trimmed region
				if (!Marker.IsLoopRegion() || Marker.FrameLength + NewPosition <= 0)
				{
					continue; // Skip adding this marker to filtered list
				}
			}

			FSoundWaveCuePoint FilteredMarker = Marker;

			if (FilteredMarker.IsLoopRegion())
			{
				// If loop region is cut, resize it to maintain proper relative loop end point
				if (NewPosition < 0 && FilteredMarker.FrameLength + NewPosition > 0)
				{
					FilteredMarker.FrameLength += NewPosition;
				}

				FilteredMarker.FrameLength = FMath::Max(FilteredMarker.FrameLength, UWaveCueArray::MinLoopSize);
			}

			FilteredMarker.FramePosition = FMath::Max(NewPosition, 0);
			FilteredAndAdjustedCuePoints.Add(FilteredMarker);
		}
		
		return FilteredAndAdjustedCuePoints;
	}
}

FWaveTransformationMarkers::FWaveTransformationMarkers(double InStartLoopTime, double InEndLoopTime)
: StartLoopTime(InStartLoopTime)
, EndLoopTime(InEndLoopTime){}

void FWaveTransformationMarkers::ProcessAudio(Audio::FWaveformTransformationWaveInfo& InOutWaveInfo) const
{
	check(InOutWaveInfo.SampleRate > 0.f && InOutWaveInfo.Audio != nullptr);

	Audio::FAlignedFloatBuffer& InputAudio = *InOutWaveInfo.Audio;

	// If InputAudio is smaller than the size of a frame, do not process the audio.
	check(InOutWaveInfo.NumChannels > 0);

	// Note FAlignedFloatBuffer max number of samples is INT32_MAX
	int32 InputAudioNum = InputAudio.Num();
	const int32 ExtraSamples = InputAudioNum % InOutWaveInfo.NumChannels;
	if (ExtraSamples > 0)
	{
		InputAudioNum -= ExtraSamples; // trim out samples beyond last full frame

		UE_LOGF(LogWaveformTransformation, Log, "Invalid number of Samples, number of samples not divisible by the channel count.");
	}

	if (InputAudioNum < InOutWaveInfo.NumChannels)
	{
		return;
	}

	check(StartLoopTime >= 0);
	check(InOutWaveInfo.NumChannels > 0);

	const int32 LastInputAudioIndex = InputAudioNum - 1;
	const int32 NumChannelsMinusOne = InOutWaveInfo.NumChannels - 1; // Used to step backwards from the end sample to a valid frame
	const int32 FirstSampleOfLastFrame = LastInputAudioIndex - NumChannelsMinusOne; // The last sample that begins a frame
	int32 StartSample = FMath::Min(FMath::FloorToInt32(StartLoopTime * InOutWaveInfo.SampleRate) * InOutWaveInfo.NumChannels, FirstSampleOfLastFrame);
	int32 EndSample = LastInputAudioIndex;
	check(StartSample <= LastInputAudioIndex);

	if (EndLoopTime > 0.f)
	{
		const int32 EndFrame = FMath::RoundToInt32(EndLoopTime * InOutWaveInfo.SampleRate);
		EndSample = EndFrame * InOutWaveInfo.NumChannels + NumChannelsMinusOne;

		// EndLoopTime can be beyond the length of the file if there is a Trim
		EndSample = FMath::Min(EndSample, LastInputAudioIndex);
	}

	if (StartSample > EndSample - NumChannelsMinusOne)
	{
		StartSample = FMath::Max(EndSample - NumChannelsMinusOne, 0);
	}

	const int32 FinalSize = (EndSample - StartSample) + 1;
	check(StartSample + FinalSize <= InputAudioNum);
	check(FinalSize % InOutWaveInfo.NumChannels == 0);
	check(StartSample >= 0);
	// Asserted here because it is validated on load
	ensure(StartSample % InOutWaveInfo.NumChannels == 0);

	InOutWaveInfo.StartSampleOffset = StartSample;
	InOutWaveInfo.NumEditedSamples = FinalSize;

	// Audio needs no trimming
	if (FinalSize == InputAudioNum)
	{
		return;
	}

	// Apply trim to the audio to audition the desired loop region
	TArray<float> TempBuffer;
	TempBuffer.AddUninitialized(FinalSize);

	FMemory::Memcpy(TempBuffer.GetData(), &InputAudio[StartSample], FinalSize * sizeof(float));

	InputAudio.Empty();
	InputAudio.AddUninitialized(FinalSize);

	FMemory::Memcpy(InputAudio.GetData(), TempBuffer.GetData(), FinalSize * sizeof(float));
}

UWaveformTransformationMarkers::UWaveformTransformationMarkers(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	if(Markers == nullptr)
	{
		Markers = ObjectInitializer.CreateDefaultSubobject<UWaveCueArray>(this, TEXT("Markers"));
	}

	Markers->CueChanged.BindLambda([this]() { SetLoopPreviewing(); });
}

#if WITH_EDITOR
void UWaveformTransformationMarkers::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformTransformationMarkers, Markers))
	{
		check(Markers);
		SetLoopPreviewing();
	}
	else 
	{
		OnTransformationChanged.ExecuteIfBound(true);
	}
}

void UWaveformTransformationMarkers::PostEditUndo()
{
	AdjustLoopPreviewIfNotAligned();

	Super::PostEditUndo();
}
#endif // WITH_EDITOR

Audio::FTransformationPtr UWaveformTransformationMarkers::CreateTransformation() const
{
	return MakeUnique<FWaveTransformationMarkers>(StartLoopTime, EndLoopTime);
}

void UWaveformTransformationMarkers::UpdateConfiguration(FWaveTransformUObjectConfiguration& InOutConfiguration)
{
	check(Markers != nullptr);
	Markers->InitMarkersIfNotSet(InOutConfiguration.WaveCues);
	
	const float InOutDuration = InOutConfiguration.EndTime - InOutConfiguration.StartTime;

	// Assert that InOutConfiguration is initialized and valid
	check(InOutConfiguration.SampleRate > 0.f);
	check(InOutConfiguration.StartTime >= 0.f);
	check(InOutDuration > 0.f);

	StartFrameOffset = static_cast<int64>(InOutConfiguration.StartTime * InOutConfiguration.SampleRate);
	SampleRate = InOutConfiguration.SampleRate;
	AvailableWaveformDuration = InOutDuration;
	NumChannels = InOutConfiguration.NumChannels;
	NumAvailableSamples = static_cast<int64>(InOutDuration * SampleRate * NumChannels);

	if (!bHasLoggedCutByLoop)
	{
		for (FSoundWaveCuePoint& CuePoint : Markers->CuesAndLoops)
		{
			if (!CuePoint.IsLoopRegion())
			{
				continue;
			}

			const int64 LoopStartSample = CuePoint.FramePosition;
			const int64 LoopEndSample = CuePoint.FramePosition + CuePoint.FrameLength;
			const int64 EndSample = FMath::RoundToInt64(InOutConfiguration.EndTime * SampleRate) * NumChannels;

			if ((StartFrameOffset > LoopStartSample && StartFrameOffset < LoopEndSample) || (LoopEndSample > EndSample && LoopStartSample < LoopEndSample))
			{
				bHasLoggedCutByLoop = true;

				UE_LOGF(LogWaveformTransformation, Log, "Cutting a loop point with a trim!");
			}
		}
	}

	if (!bCachedIsPreviewingLoopRegion)
	{
		bCachedSoundWaveLoopState = InOutConfiguration.bCachedSoundWaveLoopState;
	}
	else
	{
		InOutConfiguration.StartTime = StartLoopTime;
	}

	// Update after setting bCachedSoundWaveLoopState so bCachedSoundWaveLoopState isn't overwritten
	bCachedIsPreviewingLoopRegion = bIsPreviewingLoopRegion;

	InOutConfiguration.bIsPreviewingLoopRegion = bIsPreviewingLoopRegion;
	InOutConfiguration.bCachedSoundWaveLoopState = bCachedSoundWaveLoopState;
}

void UWaveformTransformationMarkers::OverwriteTransformation()
{
	check(Markers != nullptr);
	Markers->Reset();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void UWaveformTransformationMarkers::ModifyMarkerLoopRegion(ELoopModificationControls Modification) const
{
	check(Markers != nullptr);
	Markers->ModifyMarkerLoop.ExecuteIfBound(Modification);
}

void UWaveformTransformationMarkers::CycleMarkerLoopRegion(ELoopModificationControls Modification) const
{
	check(Markers != nullptr);
	Markers->CycleMarkerLoop.ExecuteIfBound(Modification);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

void UWaveformTransformationMarkers::ModifyMarkerLoopRegion(ELoopModificationControls Modification, const TArray<float>& PCMArray, float NoiseThreshold)
{
	check(Markers != nullptr);

	FSoundWaveCuePoint* LoopRegion = GetSelectedMarker();

	if (LoopRegion == nullptr)
	{
		return;
	}

	check(Markers);
	check(NumChannels > 0);
	check(NumAvailableSamples > 0);
	check(SizeIncrements.Num() > 0);
	check(SelectedIncrement < SizeIncrements.Num() && SelectedIncrement >= 0);

	const int64 FramesToShift = static_cast<int64>((SizeIncrements[SelectedIncrement] / 1000.0f) * SampleRate);

	if (GEditor && GEditor->Trans)
	{
		GEditor->BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("ModifyingMarkerFrame", "Modifying Marker"), nullptr);
	}

	Markers->Modify();
	const int64 MaxTotalFrames = static_cast<int64>(NumAvailableSamples) / static_cast<int64>(NumChannels);
	check(MaxTotalFrames > 0);

	switch (Modification)
	{
	case ELoopModificationControls::None:
		break;
	case ELoopModificationControls::LeftHandleIncrement:
	{
		LoopRegion->FramePosition -= FramesToShift;
		LoopRegion->FrameLength += FramesToShift;
	}
	break;
	case ELoopModificationControls::LeftHandleDecrement:
	{
		LoopRegion->FramePosition += FramesToShift;
		LoopRegion->FrameLength -= FramesToShift;
	}
	break;
	case ELoopModificationControls::RightHandleIncrement:
	{
		LoopRegion->FrameLength += FramesToShift;
	}
	break;
	case ELoopModificationControls::RightHandleDecrement:
	{
		LoopRegion->FrameLength -= FramesToShift;
	}
	break;
	case ELoopModificationControls::IncreaseIncrement:
	{
		if (SelectedIncrement < SizeIncrements.Num() - 1)
		{
			++SelectedIncrement;
		}
	}
	break;
	case ELoopModificationControls::DecreaseIncrement:
	{
		if (SelectedIncrement > 0)
		{
			--SelectedIncrement;
		}
	}
	break;
	case ELoopModificationControls::LeftHandleNextZeroCrossing:
	{
		const int64 StartFrame = (LoopRegion->FramePosition);
		const int64 EndFrame = 0;

		const int64 FramesToMove = GetFramesToNextZeroCrossing(StartFrame, EndFrame, PCMArray, false, NoiseThreshold, NumChannels);

		LoopRegion->FramePosition -= FramesToMove;
		LoopRegion->FrameLength += FramesToMove;
	}
	break;
	case ELoopModificationControls::LeftHandlePreviousZeroCrossing:
	{
		const int64 StartFrame = (LoopRegion->FramePosition);
		// Clamp end frame to max total frames
		const int64 EndFrame = FMath::Min((LoopRegion->FramePosition + LoopRegion->FrameLength), MaxTotalFrames-1);

		const int64 FramesToMove = GetFramesToNextZeroCrossing(StartFrame, EndFrame, PCMArray, true, NoiseThreshold, NumChannels);

		LoopRegion->FramePosition += FramesToMove;
		LoopRegion->FrameLength -= FramesToMove;
	}
	break;
	case ELoopModificationControls::RightHandleNextZeroCrossing:
	{
		// Clamp end frame to max total frames
		const int64 StartFrame = FMath::Min((LoopRegion->FramePosition + LoopRegion->FrameLength), MaxTotalFrames - 1);
		const int64 EndFrame = MaxTotalFrames - 1;

		LoopRegion->FrameLength += GetFramesToNextZeroCrossing(StartFrame, EndFrame, PCMArray, true, NoiseThreshold, NumChannels);
	}
	break;
	case ELoopModificationControls::RightHandlePreviousZeroCrossing:
	{
		const int64 StartFrame = FMath::Min((LoopRegion->FramePosition + LoopRegion->FrameLength), MaxTotalFrames - 1);
		const int64 EndFrame = LoopRegion->FramePosition;

		LoopRegion->FrameLength -= GetFramesToNextZeroCrossing(StartFrame, EndFrame, PCMArray, false, NoiseThreshold, NumChannels);
	}
	default:
		break;
	}

	// Prevent the frame position from exceeding the wave length
	LoopRegion->FramePosition = FMath::Clamp(LoopRegion->FramePosition, 0, MaxTotalFrames - 1);
	// FrameLength can exceed MaxFrames because the loop will be cut by ProcessAudio
	// but length must be >= MinLoopSize or it is no longer a loop
	LoopRegion->FrameLength = FMath::Max(LoopRegion->FrameLength, Markers->MinLoopSize);

	PreviewSelectedLoop();

	if (GEditor && GEditor->Trans)
	{
		GEditor->EndTransaction();
	}

	OnTransformationChanged.ExecuteIfBound(true);
}

void UWaveformTransformationMarkers::CycleMarkerLoopRegion(ELoopModificationControls Modification)
{
	check(Markers != nullptr);

	const int32 MarkerArrayLength = Markers->CuesAndLoops.Num();

	FSoundWaveCuePoint* SelectedCuePointPtr = nullptr;

	// If there is no active selection, pick first or last elmeent
	if (Markers->SelectedCue == INDEX_NONE && MarkerArrayLength > 0)
	{
		switch (Modification)
		{
		case ELoopModificationControls::SelectNextLoop:
		{
			Markers->SelectedCue = Markers->CuesAndLoops[0].CuePointID;
			SelectedCuePointPtr = &Markers->CuesAndLoops[0];
		}
		break;
		case ELoopModificationControls::SelectPreviousLoop:
		{
			Markers->SelectedCue = Markers->CuesAndLoops[MarkerArrayLength - 1].CuePointID;
			SelectedCuePointPtr = &Markers->CuesAndLoops[MarkerArrayLength - 1];
		}
		break;
		default:
			break;
		}
		return;
	}

	// Since cues can be removed and added the ids do not match list indexes, so search for matches
	for (int i = 0; i < MarkerArrayLength; i++)
	{
		if (Markers->CuesAndLoops[i].CuePointID == Markers->SelectedCue)
		{
			// Check if we move backwards of forwards
			if (Modification == ELoopModificationControls::SelectNextLoop)
			{
				Markers->SelectedCue = Markers->CuesAndLoops[(i + 1) % MarkerArrayLength].CuePointID;
				SelectedCuePointPtr = &Markers->CuesAndLoops[(i + 1) % MarkerArrayLength];
				break;
			}
			else if (Modification == ELoopModificationControls::SelectPreviousLoop)
			{
				Markers->SelectedCue = Markers->CuesAndLoops[(i - 1 + MarkerArrayLength) % MarkerArrayLength].CuePointID;
				SelectedCuePointPtr = &Markers->CuesAndLoops[(i - 1 + MarkerArrayLength) % MarkerArrayLength];
				break;
			}
		}
	}

	if (SelectedCuePointPtr && SelectedCuePointPtr->IsLoopRegion())
	{
		PreviewSelectedLoop();
	}
	else
	{
		ResetLoopPreviewing();
	}

	OnTransformationChanged.ExecuteIfBound(false);
}

int64 UWaveformTransformationMarkers::GetFramesToNextZeroCrossing(int64 StartFrame, int64 EndFrame, const TArray<float>& PCMArray, bool bCheckIncrementing, float NoiseThreshold, int ChannelCount)
{
	check(ChannelCount > 0);
	check(PCMArray.Num() > 0);

	check(StartFrame >= 0);
	check(StartFrame <= (PCMArray.Num() / ChannelCount) - 1);
	check(EndFrame >= 0);
	check(EndFrame <= (PCMArray.Num() / ChannelCount) - 1);

	int64 FramesMoved = 0;

	bool bAllSamplesPositiveLastHighNoise = false;
	bool bAllSamplesNegativeLastHighNoise = false;

	// Check for clamping within array bounds
	StartFrame = FMath::Clamp(StartFrame, 0, (PCMArray.Num() / ChannelCount) - 1);
	EndFrame = FMath::Clamp(EndFrame, 0, (PCMArray.Num() / ChannelCount) - 1);

	if (bCheckIncrementing)
	{
		check(EndFrame >= StartFrame);

		for (int64 i = StartFrame; i < EndFrame; i++)
		{
			if (CheckZeroCrossingFrame(i * ChannelCount, PCMArray, bAllSamplesPositiveLastHighNoise, bAllSamplesNegativeLastHighNoise, NoiseThreshold, ChannelCount))
			{
				return FramesMoved;
			}

			FramesMoved++;
		}
	}
	else
	{
		check(StartFrame >= EndFrame);

		for (int64 i = StartFrame; i >= EndFrame; i--)
		{
			if (CheckZeroCrossingFrame(i * ChannelCount, PCMArray, bAllSamplesPositiveLastHighNoise, bAllSamplesNegativeLastHighNoise, NoiseThreshold, ChannelCount))
			{
				return FramesMoved;
			}

			FramesMoved++;
		}
	}

	return 0;
}

int64 UWaveformTransformationMarkers::CheckZeroCrossingFrame(int64 CurrentSample, const TArray<float>& PCMArray, bool& bAllSamplesPositiveLastHighNoise, bool& bAllSamplesNegativeLastHighNoise, float NoiseThreshold, int ChannelCount)
{
	check(ChannelCount > 0);
	check(CurrentSample % ChannelCount == 0); // Check for frame alignment 

	if (CurrentSample + ChannelCount > PCMArray.Num())
	{
		return 0;
	}

	int32 PositiveSampleCount = 0;
	int32 NegativeSampleCount = 0;
	int32 LowNoiseSampleCount = 0;

	// check if all channels have a zero crossing
	for (int32 ChannelNum = 0; ChannelNum < ChannelCount; ChannelNum++)
	{
		const float FrameValue = PCMArray[CurrentSample + ChannelNum];

		// Treat values within the noise threshold as valid for both negative and positive
		if (FrameValue >= -NoiseThreshold && FrameValue <= NoiseThreshold)
		{
			++NegativeSampleCount;
			++PositiveSampleCount;
			++LowNoiseSampleCount;
		}
		else if (FrameValue < 0)
		{
			++NegativeSampleCount;
		}
		else
		{
			++PositiveSampleCount;
		}
	}

	bool bAllSamplesPositive = PositiveSampleCount == ChannelCount;
	bool bAllSamplesNegative = NegativeSampleCount == ChannelCount;
	bool bAllSamplesLowNoise = LowNoiseSampleCount == ChannelCount;

	// Check if we have crossed the zero crossing on all channels between last frame and this frame
	// If we are in a low noise frame then we ignore it for zero crossing
	bool bZeroCrossingOccurred = (bAllSamplesPositive && bAllSamplesNegativeLastHighNoise && !bAllSamplesLowNoise) || (bAllSamplesNegative && bAllSamplesPositiveLastHighNoise && !bAllSamplesLowNoise);

	if (!bAllSamplesLowNoise)
	{
		bAllSamplesPositiveLastHighNoise = bAllSamplesPositive;
		bAllSamplesNegativeLastHighNoise = bAllSamplesNegative;
	}

	return bZeroCrossingOccurred;
}

bool UWaveformTransformationMarkers::IsLoopRegionActive() const
{
	if (bIsPreviewingLoopRegion || EndLoopTime != -1.0)
	{
		return true;
	}

	return false;
}

void UWaveformTransformationMarkers::SetLoopPreviewing()
{
	check(Markers);
	bool bIsSelectedCuePresent = false;
	bool bTransformationChanged = false;
	bool bTransformationChangedWithoutMarkingDirty = false;

	for (FSoundWaveCuePoint& CuePoint : Markers->CuesAndLoops)
	{
		if (!bIsSelectedCuePresent && Markers->SelectedCue == CuePoint.CuePointID)
		{
			bIsSelectedCuePresent = true;

			if (!CuePoint.IsLoopRegion())
			{
				bTransformationChanged = ResetLoopPreviewing();
			}
		}

		if (CuePoint.IsLoopRegion())
		{
			if (CuePoint.FrameLength < Markers->MinLoopSize)
			{
				CuePoint.FrameLength = FMath::Max(Markers->MinLoopSize, static_cast<int64>(AvailableWaveformDuration * SampleRate * 0.1f));
				bTransformationChanged = true;
			}

			if (Markers->SelectedCue == CuePoint.CuePointID && !IsLoopRegionActive())
			{
				bTransformationChangedWithoutMarkingDirty = true;
				PreviewSelectedLoop();
			}

			// Update LoopRegion Preview window if a loop is selected and the user edits the loop bounds using the properties window
			if (IsLoopRegionActive() && Markers->SelectedCue == CuePoint.CuePointID)
			{
				check(SampleRate > 0.f);
				check(EndLoopTime > 0.0);

				const int64 StartLoopFramePos = (StartLoopTime * SampleRate);
				const int64 EndLoopFramePos = (EndLoopTime * SampleRate);

				if (CuePoint.FramePosition != StartLoopFramePos)
				{
					StartLoopTime = static_cast<float>(FMath::Max(0, CuePoint.FramePosition - StartFrameOffset)) / SampleRate;
					check(StartLoopTime >= 0);
					bTransformationChanged = true;
				}

				if (CuePoint.FramePosition + CuePoint.FrameLength != EndLoopFramePos)
				{
					EndLoopTime = static_cast<float>(CuePoint.FramePosition + CuePoint.FrameLength) / SampleRate;
					bTransformationChanged = true;
				}
			}
		}
	}

	if (!bIsSelectedCuePresent && Markers->SelectedCue != INDEX_NONE)
	{
		Markers->SelectedCue = INDEX_NONE;
	}

	if (Markers->SelectedCue == INDEX_NONE)
	{
		bTransformationChanged = ResetLoopPreviewing();
	}
}

bool UWaveformTransformationMarkers::ResetLoopPreviewing()
{
	if (IsLoopRegionActive())
	{
		bIsPreviewingLoopRegion = false;
		StartLoopTime = 0.0;
		EndLoopTime = -1.0;

		return true;
	}

	return false;
}

bool UWaveformTransformationMarkers::AdjustLoopPreviewIfNotAligned()
{
	check(Markers);
	FSoundWaveCuePoint* CuePoint = GetSelectedMarker();

	if (Markers->SelectedCue == INDEX_NONE || !IsLoopRegionActive() || CuePoint == nullptr)
	{
		return false;
	}

	check(SampleRate > 0.f);

	const int64 StartLoopFramePos = static_cast<int64>(StartLoopTime * SampleRate);
	const int64 EndLoopFramePos = static_cast<int64>(EndLoopTime * SampleRate);

	bool bPreviewChanged = false;

	const int64 CuePointPosDif = FMath::Abs(CuePoint->FramePosition - StartLoopFramePos - StartFrameOffset);
	const int64 CuePointEndPosDif = FMath::Abs((CuePoint->FramePosition + CuePoint->FrameLength) - EndLoopFramePos - StartFrameOffset);

	// Converting from time to frame is not perfectly accurate, lets be +- 2 frames
	if (CuePointPosDif > 2)
	{
		const double PrevStartLoopTime = StartLoopTime;
		StartLoopTime = static_cast<float>(FMath::Max(0, CuePoint->FramePosition - StartFrameOffset)) / SampleRate;
		bPreviewChanged = (StartLoopTime != PrevStartLoopTime);
	}

	// Converting from time to frame is not perfectly accurate, lets be +- 2 frames
	if (CuePointEndPosDif > 2)
	{
		const double PrevEndLoopTime = EndLoopTime;
		EndLoopTime = static_cast<float>((CuePoint->FramePosition + CuePoint->FrameLength) - StartFrameOffset) / SampleRate;
		bPreviewChanged |= (EndLoopTime != PrevEndLoopTime);
	}

	return bPreviewChanged;
}

#if WITH_EDITOR
void UWaveformTransformationMarkers::OverwriteSoundWaveData(USoundWave& InOutSoundWave)
{
	// Users can export a loop region, shift FramePositions relative to the Loop region
	if (bIsPreviewingLoopRegion)
	{
		check(InOutSoundWave.GetImportedSampleRate() > 0);
		int64 StartLoopFramePosition = StartLoopTime * InOutSoundWave.GetImportedSampleRate();
		StartFrameOffset = StartFrameOffset < StartLoopFramePosition ? StartLoopFramePosition : StartFrameOffset;
	}

	// Overwriting soundwave data can cause a change in number of samples, invalidating the FramePositions of CuesAndLoops
	// Subtracting StartFrameOffset from FramePosition shifts the CuesAndLoops to the correct relative position
	const TArray<FSoundWaveCuePoint>& FrameAdjustedCuesAndLoops = GetAdjustedCuePoints(Markers->CuesAndLoops, StartFrameOffset);

	InOutSoundWave.SetSoundWaveCuePoints(FrameAdjustedCuesAndLoops);
}

void UWaveformTransformationMarkers::GetTransformationInfo(FWaveformTransformationInfo& InOutTransformationInfo) const
{
	check(Markers != nullptr);
	InOutTransformationInfo.AllCuePoints.Append(Markers->CuesAndLoops);

	constexpr bool bRemoveTrimmed = true;
	InOutTransformationInfo.AllCuePointsRelativeToStartTime.Append(GetAdjustedCuePoints(Markers->CuesAndLoops, StartFrameOffset, bRemoveTrimmed));
}

FString UWaveformTransformationMarkers::GetTransformationHash() const
{
	check(Markers);

	using FPCU = FPlatformCompressionUtilities;
	FString Hash;
	
	for (const FSoundWaveCuePoint& CuePoint : Markers->CuesAndLoops)
	{
		FPCU::AppendHash(Hash, TEXT("CPID"), CuePoint.CuePointID);
		FPCU::AppendHash(Hash, TEXT("CPFP"), CuePoint.FramePosition);
		FPCU::AppendHash(Hash, TEXT("CPFL"), CuePoint.FrameLength);
		FPCU::AppendHash(Hash, TEXT("CPILR"), CuePoint.IsLoopRegion());
		FPCU::AppendHash(Hash, TEXT("CPL"), CuePoint.Label);
	}

	return Hash;
}

FSoundWaveCuePoint* UWaveformTransformationMarkers::GetSelectedMarker() const
{
	check(Markers);

	if (Markers->SelectedCue == INDEX_NONE)
	{
		return nullptr;
	}

	for (int i = 0; i < Markers->CuesAndLoops.Num(); i++)
	{
		if (Markers->CuesAndLoops[i].CuePointID == Markers->SelectedCue)
		{
			return &Markers->CuesAndLoops[i];
		}
	}

	return nullptr;
}

void UWaveformTransformationMarkers::PreviewSelectedLoop()
{
	check(SampleRate > 0);
	check(NumChannels > 0);

	const int64 EndFramePosition = StartFrameOffset + (NumAvailableSamples / NumChannels);

	for (FSoundWaveCuePoint CuePoint : Markers->CuesAndLoops)
	{
		const int64 LoopEndFramePosition = CuePoint.FramePosition + CuePoint.FrameLength;

		// Only preview a loop if part of it is within the available frames (accounting for TrimFades and other transformations)
		if (CuePoint.CuePointID == Markers->SelectedCue && CuePoint.IsLoopRegion() &&
			LoopEndFramePosition >= StartFrameOffset && CuePoint.FramePosition < EndFramePosition)
		{
			check(CuePoint.FrameLength > 0);

			// Set Loop preview handles
			const float StartLoopPosInSeconds = static_cast<float>(CuePoint.FramePosition) / SampleRate;
			const float EndLoopPosInSeconds = static_cast<float>(LoopEndFramePosition) / SampleRate;

			SetIsPreviewingLoopRegion(static_cast<double>(StartLoopPosInSeconds), static_cast<double>(EndLoopPosInSeconds), true);
		}
	}
}

void UWaveformTransformationMarkers::SetIsPreviewingLoopRegion(double InStartTime, double InEndTime, bool bIsPreviewing)
{
	// Stop any current loop previewing so StartLoopTime is never greater than EndLoopTime (Other than when EndLoopTime is invalid)
	ResetLoopPreviewing();

	bIsPreviewingLoopRegion = bIsPreviewing;
	StartLoopTime = InStartTime;
	EndLoopTime = InEndTime;
}

void UWaveformTransformationMarkers::SelectLoopRegionByKeyboard(const FKey& PressedKey)
{
	check(Markers);

	for (int i = 0; i < IndexKeyCodes.Num(); i++)
	{
		// Check we have loop regions in this index
		if (Markers->CuesAndLoops.Num() <= i)
		{
			return;
		}

		if (PressedKey == IndexKeyCodes[i])
		{
			Markers->SelectedCue = i;

			check(SampleRate > 0);
			if (GetSelectedMarker() && GetSelectedMarker()->IsLoopRegion())
			{
				PreviewSelectedLoop();
			}
			else
			{
				SetIsPreviewingLoopRegion(0.0, -1.0, false);
			}
		}
	}
}

#if WITH_EDITOR
void UWaveCueArray::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveCueArray, CuesAndLoops) && CuesAndLoops.Num() > 0)
	{
		int32 LargestCuePointID = 1;

		for (int32 Index = CuesAndLoops.Num() - 1; Index >= 0; Index--)
		{
			if (CuesAndLoops[Index].CuePointID >= LargestCuePointID)
			{
				LargestCuePointID = CuesAndLoops[Index].CuePointID + 1;
			}
		}

		// When an element is added or reset to default, CuePointID is equal to INDEX_NONE so only those cue points have to be addressed
		if (PropertyChangedEvent.ChangeType & (EPropertyChangeType::ArrayAdd | EPropertyChangeType::ResetToDefault |
				EPropertyChangeType::Unspecified | EPropertyChangeType::Redirected))
		{
			// Elements can be inserted at any index in the array
			for (FSoundWaveCuePoint& CuePoint : CuesAndLoops)
			{
				if (CuePoint.CuePointID == INDEX_NONE)
				{
					CuePoint.CuePointID = LargestCuePointID++;
				}
			}
		}
		// When an element is duplicated, the CuePointID is also duplicated so the new cue point needs a new unique ID
		// Only works for adjacent indicies which is sufficient for the way Duplicate works in the details panel
		else if (PropertyChangedEvent.ChangeType & EPropertyChangeType::Duplicate)
		{
			int32 PrevCuePointID = INDEX_NONE;

			for (FSoundWaveCuePoint& CuePoint : CuesAndLoops)
			{
				if (CuePoint.CuePointID == PrevCuePointID)
				{
					CuePoint.CuePointID = LargestCuePointID++;
				}

				PrevCuePointID = CuePoint.CuePointID;
			}
		}
	}

	CueChanged.ExecuteIfBound();
}

void UWaveCueArray::PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	// Clear selection if property change type is ArrayRemove, ArrayClear, ToggleEditable (edit condition state has changed), or Unspecified.
	SelectedCue = INDEX_NONE;
	if ( PropertyChangedEvent.ChangeType & (EPropertyChangeType::ArrayAdd
										   |EPropertyChangeType::ResetToDefault
										   |EPropertyChangeType::ValueSet
										   |EPropertyChangeType::Duplicate
										   |EPropertyChangeType::Interactive
										   |EPropertyChangeType::ArrayMove
										   |EPropertyChangeType::Redirected) )
	{
		if (CuesAndLoops.Num() > 0 && PropertyChangedEvent.PropertyChain.GetActiveMemberNode() != nullptr)
		{
			int32 MemberNodeIndex = PropertyChangedEvent.GetArrayIndex(PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue()->GetName());
			if (MemberNodeIndex >= 0 && MemberNodeIndex < CuesAndLoops.Num())
			{
				SelectedCue = CuesAndLoops[MemberNodeIndex].CuePointID;
			}
		}
	}
}
#endif // WITH_EDITOR

void UWaveCueArray::InitMarkersIfNotSet(const TArray<FSoundWaveCuePoint>& InMarkers)
{
	//Prevent USoundWave from overwriting the Transformation unintentionally
	if (!bIsInitialized)
	{
		CuesAndLoops = InMarkers;
		bIsInitialized = true;
	}
}

void UWaveCueArray::Reset()
{
	CuesAndLoops.Empty();
	bIsInitialized = false;
}
#endif

void UWaveCueArray::EnableLoopRegion(FSoundWaveCuePoint* OutSoundWaveCue, bool bSetLoopRegion)
{
	check(OutSoundWaveCue != nullptr);
	OutSoundWaveCue->SetLoopRegion(bSetLoopRegion);
}

#undef LOCTEXT_NAMESPACE