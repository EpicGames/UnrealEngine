// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationMarkersObjectCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "WaveformTransformationMarkers.h"

TSharedRef<IPropertyTypeCustomization> FWaveformTransformationMarkersObjectCustomization::MakeInstance()
{
	return MakeShareable(new FWaveformTransformationMarkersObjectCustomization);
}

void FWaveformTransformationMarkersObjectCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle,
	FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	constexpr bool bDisplayDefaultPropertyButtons = false;

	HeaderRow.NameContent()[PropertyHandle->CreatePropertyNameWidget()]
	.ValueContent()[PropertyHandle->CreatePropertyValueWidget(bDisplayDefaultPropertyButtons)];
}

void FWaveformTransformationMarkersObjectCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle,
	IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	TSharedPtr<IPropertyHandle> MarkersHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UWaveformTransformationMarkers, Markers));

	if (!MarkersHandle || !MarkersHandle->IsValidHandle())
	{
		return;
	}

	// Only show the UWaveCueArray property, and nothing else
	TSharedPtr<IPropertyHandle> WaveCueMarkersHandle = MarkersHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UWaveCueArray, CuesAndLoops));
	if (WaveCueMarkersHandle && WaveCueMarkersHandle->IsValidHandle())
	{
		ChildBuilder.AddProperty(WaveCueMarkersHandle.ToSharedRef());
	}
}