// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationNormalize.h"

#include "AudioCompressionSettingsUtils.h"
#include "DSP/Dsp.h"
#include "DSP/FloatArrayMath.h"
#include "WaveformAudioAnalysisFunctions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveformTransformationNormalize)

FWaveTransformationNormalize::FWaveTransformationNormalize(float InTarget, float InMaxGain, ENormalizationMode InMode)
	: Target(InTarget)
	, MaxGain(InMaxGain)
	, Mode(InMode) {}


 void FWaveTransformationNormalize::ProcessAudio(Audio::FWaveformTransformationWaveInfo& InOutWaveInfo) const
{
	check(InOutWaveInfo.Audio != nullptr);
	
	Audio::FAlignedFloatBuffer& InputAudio = *InOutWaveInfo.Audio;

	float PeakDecibelValue = 1.f;
	
	switch (Mode)
	{
	case ENormalizationMode::Peak:
		{
			const float PeakLinearValue = Audio::ArrayMaxAbsValue(InputAudio);
			PeakDecibelValue = Audio::ConvertToDecibels(PeakLinearValue);
			break;
		}
	case ENormalizationMode::RMS:
		PeakDecibelValue = WaveformAudioAnalysis::GetRMSPeak(InputAudio, InOutWaveInfo.SampleRate, InOutWaveInfo.NumChannels);
		PeakDecibelValue = Audio::ConvertToDecibels(PeakDecibelValue);
		break;
	case ENormalizationMode::DWeightedLoudness:
		PeakDecibelValue = WaveformAudioAnalysis::GetLoudnessPeak(InputAudio,  InOutWaveInfo.SampleRate, InOutWaveInfo.NumChannels);
		break;
	case ENormalizationMode::COUNT:
	default:
		return;
	}

	const float TargetOffset = Target - PeakDecibelValue;
	const float MakeupGain = FMath::Clamp(TargetOffset, -MaxGain, MaxGain);
	const float LinearPeakSample = Audio::ArrayMaxAbsValue(InputAudio);
	InOutWaveInfo.StartSampleOffset = 0;

	if (MakeupGain != 0.f && LinearPeakSample > 0.f)
	{
		// Prevent clipping from over amplification
		const float MaxMakeupGain = 1.f / LinearPeakSample;
		const float ClampedLinearMakeupGain = FMath::Min(Audio::ConvertToLinear(MakeupGain), MaxMakeupGain);

		Audio::ArrayMultiplyByConstantInPlace(InputAudio, ClampedLinearMakeupGain);
	}
}

Audio::FTransformationPtr UWaveformTransformationNormalize::CreateTransformation() const
{
	return MakeUnique<FWaveTransformationNormalize>(Target, MaxGain, Mode);
}

#if WITH_EDITOR
FString UWaveformTransformationNormalize::GetTransformationHash() const
{
	using FPCU = FPlatformCompressionUtilities;
	FString Hash;

	FPCU::AppendHash(Hash, TEXT("NMG"),MaxGain);
	FPCU::AppendHash(Hash, TEXT("NT"), Target);
	FPCU::AppendHash(Hash, TEXT("NM"), static_cast<uint8>(Mode));

	return Hash;
}
#endif //WITH_EDITOR