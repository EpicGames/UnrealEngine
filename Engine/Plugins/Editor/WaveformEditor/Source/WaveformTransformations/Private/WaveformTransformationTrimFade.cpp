// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationTrimFade.h"

#include "AudioCompressionSettingsUtils.h"
#include "Editor.h"
#include "Sound/SoundWave.h"
#include "WaveformTransformationLog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveformTransformationTrimFade)

namespace WaveformTrasnformationTrimFadePrivate
{
	// Makes the two functions for SCurve meet at (0.5, 0.5)
	const double XMultiplier = FMath::Sqrt(10.0) / 5.0;

	constexpr float Linear = 1.f;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static void SetFadeCurve(TObjectPtr<UFadeFunction> FadeFunction, float FadeCurve, float Sharpness)
	{
		if (FadeFunction->IsA<UFadeCurveFunction>())
		{
			TObjectPtr<UFadeCurveFunction> CurveFunction = CastChecked<UFadeCurveFunction>(FadeFunction);

			if (CurveFunction)
			{
				if (!CurveFunction->IsA<UFadeCurveFunctionSigmoid>())
				{
					CurveFunction->SetFadeCurve(FadeCurve);
				}
				else
				{
					CurveFunction->SetFadeCurve(Sharpness);
				}
			}
			
		}
	}

	static void MigrateToTransformationFadeFunction(const TObjectPtr<UWaveformTransformationFade>& InFadeTransformation, const TObjectPtr<UFadeFunction>& InFadeFunction, TObjectPtr<UTransformationFadeFunction>& NewTransformationFadeFunction)
	{
		check(InFadeTransformation);
		check(InFadeFunction);

		bool bValidFunctionType = true;

		if (InFadeFunction->IsA<UFadeCurveFunction>())
		{
			TObjectPtr<UFadeCurveFunction> CurveFunction = CastChecked<UFadeCurveFunction>(InFadeFunction);

			if (CurveFunction)
			{
				if (CurveFunction->IsA<UFadeCurveFunctionExponential>())
				{
					NewTransformationFadeFunction = NewObject<UTransformationFadeCurveFunctionExponential>(InFadeTransformation, UTransformationFadeCurveFunctionExponential::StaticClass(), NAME_None, RF_Transactional);
				}
				else if (CurveFunction->IsA<UFadeCurveFunctionLogarithmic>())
				{
					NewTransformationFadeFunction = NewObject<UTransformationFadeCurveFunctionLogarithmic>(InFadeTransformation, UTransformationFadeCurveFunctionLogarithmic::StaticClass(), NAME_None, RF_Transactional);
				}
				else if (CurveFunction->IsA<UFadeCurveFunctionSigmoid>())
				{
					NewTransformationFadeFunction = NewObject<UTransformationFadeCurveFunctionSigmoid>(InFadeTransformation, UTransformationFadeCurveFunctionSigmoid::StaticClass(), NAME_None, RF_Transactional);
				}
				else
				{
					bValidFunctionType = false;
				}

				if (bValidFunctionType)
				{
					TObjectPtr<UTransformationFadeCurveFunction> TransformationCurve = CastChecked<UTransformationFadeCurveFunction>(NewTransformationFadeFunction);

					if (TransformationCurve)
					{
						TransformationCurve->SetFadeCurve(CurveFunction->GetFadeCurve());
					}
				}
			}
		}
		else if (InFadeFunction->IsA<UFadeFunctionLinear>())
		{
			NewTransformationFadeFunction = NewObject<UTransformationFadeFunctionLinear>(InFadeTransformation, UTransformationFadeFunctionLinear::StaticClass(), NAME_None, RF_Transactional);
		}
		else
		{
			bValidFunctionType = false;
		}

		if (!bValidFunctionType)
		{
			UE_LOGF(LogWaveformTransformation, Warning, "Unknown Curve Function in TrimFade Transformation Migration. Initializing it as exponential");

			NewTransformationFadeFunction = NewObject<UTransformationFadeCurveFunctionExponential>(InFadeTransformation, UTransformationFadeCurveFunctionExponential::StaticClass(), NAME_None, RF_Transactional);
		}

		NewTransformationFadeFunction->Duration = InFadeFunction->Duration;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

namespace WaveformTransformationTrimFadeNames
{
	static FLazyName StartTimeName(GET_MEMBER_NAME_CHECKED(UWaveformTransformationTrimFade, StartTime));
	static FLazyName EndTimeName(GET_MEMBER_NAME_CHECKED(UWaveformTransformationTrimFade, EndTime));
}

FWaveTransformationTrimFade::FWaveTransformationTrimFade(TObjectPtr<UWaveformTransformationFade> InFadeTransformation, double InStartTime, double InEndTime)
	: FadeTransformation(InFadeTransformation)
	, StartTime(InStartTime)
	, EndTime(InEndTime){}

void FWaveTransformationTrimFade::ProcessAudio(Audio::FWaveformTransformationWaveInfo& InOutWaveInfo) const
{
	check(InOutWaveInfo.SampleRate > 0.f && InOutWaveInfo.Audio != nullptr);

	Audio::FAlignedFloatBuffer& InputAudio = *InOutWaveInfo.Audio;

	check(InOutWaveInfo.NumChannels > 0);

	int32 InputAudioNum = InputAudio.Num();
	const int32 ExtraSamples = InputAudioNum % InOutWaveInfo.NumChannels;
	if (ExtraSamples > 0)
	{
		InputAudioNum -= ExtraSamples; // trim out samples beyond last full frame

		UE_LOGF(LogWaveformTransformation, Log, "Invalid number of Samples, number of samples not divisible by the channel count.");
	}

	// If InputAudio is smaller than the size of a frame, do not process the audio.
	if (InputAudioNum < InOutWaveInfo.NumChannels)
	{
		return;
	}

	const int32 LastInputAudioIndex = InputAudioNum - 1;
	const int32 NumChannelsMinusOne = InOutWaveInfo.NumChannels - 1; // Used to step backwards from the end sample to a valid frame
	const int32 FirstSampleOfLastFrame = LastInputAudioIndex - NumChannelsMinusOne; // The last sample that begins a frame
	int32 StartSample = FMath::Min(FMath::FloorToInt32(StartTime * InOutWaveInfo.SampleRate) * InOutWaveInfo.NumChannels, FirstSampleOfLastFrame);
	int32 EndSample = LastInputAudioIndex;
	check(StartSample <= LastInputAudioIndex);
	
	if(EndTime > 0.f)
	{	
		const int32 EndFrame = FMath::RoundToInt32(EndTime * InOutWaveInfo.SampleRate);
		EndSample = EndFrame * InOutWaveInfo.NumChannels + NumChannelsMinusOne;
		EndSample = FMath::Min(EndSample, LastInputAudioIndex);
	}

	if (StartSample > EndSample - NumChannelsMinusOne)
	{
		StartSample = FMath::Max(EndSample - NumChannelsMinusOne, 0);
	}

	check(StartSample >= 0);
	check(StartSample % InOutWaveInfo.NumChannels == 0);
	check(EndSample >= StartSample);
	check((EndSample - NumChannelsMinusOne) % InOutWaveInfo.NumChannels == 0);

	const int32 FinalSizeInSamples = (EndSample - StartSample) + 1;
	check(FinalSizeInSamples % InOutWaveInfo.NumChannels == 0);
	
	check(FinalSizeInSamples <= InputAudio.Num());
	check(FinalSizeInSamples > 0);

	const bool bApplyTrim = FinalSizeInSamples < InputAudio.Num();

	if (bApplyTrim)
	{
		FMemory::Memmove(InputAudio.GetData(), &InputAudio[StartSample], FinalSizeInSamples * sizeof(float));
		InputAudio.SetNum(FinalSizeInSamples, EAllowShrinking::Yes);
	}

	InOutWaveInfo.NumEditedSamples = InputAudio.Num();

	if (FadeTransformation)
	{
		FadeTransformation->CreateTransformation()->ProcessAudio(InOutWaveInfo);
	}

	if (bApplyTrim)
	{
		// Done after fade transformation to avoid breaking the start offset
		InOutWaveInfo.StartSampleOffset = StartSample;
	}
}

UWaveformTransformationTrimFade::UWaveformTransformationTrimFade(const FObjectInitializer& ObjectInitializer) : Super(ObjectInitializer)
{
	FadeTransformation = CreateDefaultSubobject<UWaveformTransformationFade>(TEXT("FadeTransformation"));
}

#if WITH_EDITOR
void UWaveformTransformationTrimFade::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	// Transformations are updated in Super::PostEditChangeProperty(PropertyChangedEvent);
	// When Transformations are updated, UpdateDurationProperties() validates these asserts are correct
	ensure(EndTime > StartTime);
	const float MaxDuration = EndTime - StartTime;

	if (FadeTransformation)
	{
		FadeTransformation->LimitFadeRegionsToOne();

		if (GetFadeRegion().FadeIn)
		{
			GetFadeRegion().FadeIn->Duration = FMath::Min(GetFadeRegion().FadeIn->Duration, MaxDuration);
		}

		if (GetFadeRegion().FadeOut)
		{
			GetFadeRegion().FadeOut->Duration = FMath::Min(GetFadeRegion().FadeOut->Duration, MaxDuration);
		}
	}
}

FString UWaveformTransformationTrimFade::GetTransformationHash() const
{
	using FPCU = FPlatformCompressionUtilities;
	FString Hash;

	float FadeInDuration = 0.f;
	float FadeInCurve = WaveformTrasnformationTrimFadePrivate::Linear;

	float FadeOutDuration = 0.f;
	float FadeOutCurve = WaveformTrasnformationTrimFadePrivate::Linear;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (FadeFunctions.FadeIn)
	{
		FadeInDuration = FadeFunctions.FadeIn->Duration;
		if (FadeFunctions.FadeIn.IsA<UFadeCurveFunction>())
		{
			TObjectPtr<UFadeCurveFunction> CurveFunction = CastChecked<UFadeCurveFunction>(FadeFunctions.FadeIn);
			FadeInCurve = CurveFunction->GetFadeCurve();
		}

		FPCU::AppendHash(Hash, TEXT("TFIC"), FadeInCurve);
		FPCU::AppendHash(Hash, TEXT("TFID"), FadeInDuration);
	}

	if (FadeFunctions.FadeOut)
	{
		FadeOutDuration = FadeFunctions.FadeOut->Duration;
		if (FadeFunctions.FadeOut.IsA<UFadeCurveFunction>())
		{
			TObjectPtr<UFadeCurveFunction> CurveFunction = CastChecked<UFadeCurveFunction>(FadeFunctions.FadeOut);
			FadeOutCurve = CurveFunction->GetFadeCurve();
		}

		FPCU::AppendHash(Hash, TEXT("TFOC"), FadeOutCurve);
		FPCU::AppendHash(Hash, TEXT("TFOD"), FadeOutDuration);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	if (FadeTransformation)
	{
		FPCU::AppendHash(Hash, TEXT("TFFT"), FadeTransformation->GetTransformationHash());
	}

	FPCU::AppendHash(Hash, TEXT("TFST"), StartTime);
	FPCU::AppendHash(Hash, TEXT("TFET"), EndTime);
	
	return Hash;
}
#endif // WITH_EDITOR

void UWaveformTransformationTrimFade::PostLoad()
{
	Super::PostLoad();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	// Overwrite exiting fade in function data with deprecated curve properties to migrate the data to the new properties
	if (StartFadeTime != 0.f)
	{
		if (StartFadeCurve > WaveformTrasnformationTrimFadePrivate::Linear)
		{
			FadeFunctions.FadeIn = NewObject<UFadeCurveFunctionExponential>(this, UFadeCurveFunctionExponential::StaticClass(), NAME_None, RF_Transactional);
		}
		else if (StartFadeCurve < 0.f)
		{
			FadeFunctions.FadeIn = NewObject<UFadeCurveFunctionSigmoid>(this, UFadeCurveFunctionSigmoid::StaticClass(), NAME_None, RF_Transactional);
		}
		else if (StartFadeCurve < WaveformTrasnformationTrimFadePrivate::Linear)
		{
			FadeFunctions.FadeIn = NewObject<UFadeCurveFunctionLogarithmic>(this, UFadeCurveFunctionLogarithmic::StaticClass(), NAME_None, RF_Transactional);
		}
		else if (StartFadeCurve == WaveformTrasnformationTrimFadePrivate::Linear)
		{
			FadeFunctions.FadeIn = NewObject<UFadeFunctionLinear>(this, UFadeFunctionLinear::StaticClass(), NAME_None, RF_Transactional);
		}

		FadeFunctions.FadeIn->Duration = StartFadeTime;
		WaveformTrasnformationTrimFadePrivate::SetFadeCurve(FadeFunctions.FadeIn, StartFadeCurve, StartSCurveSharpness);

		StartFadeTime = 0.f;
	}

	// Overwrite exiting fade out function data with deprecated curve properties to migrate the data to the new properties
	if (EndFadeTime != 0.f)
	{
		if (EndFadeCurve > WaveformTrasnformationTrimFadePrivate::Linear)
		{
			FadeFunctions.FadeOut = NewObject<UFadeCurveFunctionExponential>(this, UFadeCurveFunctionExponential::StaticClass(), NAME_None, RF_Transactional);
		}
		else if (EndFadeCurve < 0.f)
		{
			FadeFunctions.FadeOut = NewObject<UFadeCurveFunctionSigmoid>(this, UFadeCurveFunctionSigmoid::StaticClass(), NAME_None, RF_Transactional);
		}
		else if (EndFadeCurve < WaveformTrasnformationTrimFadePrivate::Linear)
		{
			FadeFunctions.FadeOut = NewObject<UFadeCurveFunctionLogarithmic>(this, UFadeCurveFunctionLogarithmic::StaticClass(), NAME_None, RF_Transactional);
		}
		else if (EndFadeCurve == WaveformTrasnformationTrimFadePrivate::Linear)
		{
			FadeFunctions.FadeOut = NewObject<UFadeFunctionLinear>(this, UFadeFunctionLinear::StaticClass(), NAME_None, RF_Transactional);
		}

		FadeFunctions.FadeOut->Duration = EndFadeTime;
		WaveformTrasnformationTrimFadePrivate::SetFadeCurve(FadeFunctions.FadeOut, EndFadeCurve, EndSCurveSharpness);

		EndFadeTime = 0.f;
	}

	if (FadeFunctions.FadeIn || FadeFunctions.FadeOut)
	{
		check(FadeTransformation);

		FadeFunctions.MigrateToTransformationFadeFunction(FadeTransformation, GetMutableFadeRegion());

		FadeFunctions.FadeIn = nullptr;
		FadeFunctions.FadeOut = nullptr;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

}

Audio::FTransformationPtr UWaveformTransformationTrimFade::CreateTransformation() const
{
	return MakeUnique<FWaveTransformationTrimFade>(FadeTransformation, StartTime, EndTime);
}

void UWaveformTransformationTrimFade::UpdateConfiguration(FWaveTransformUObjectConfiguration& InOutConfiguration)
{
	check(InOutConfiguration.SampleRate > 0.f);
	UpdateDurationProperties(InOutConfiguration.EndTime - InOutConfiguration.StartTime);

	InOutConfiguration.StartTime = StartTime;
	InOutConfiguration.EndTime = EndTime;

	// FadeTransformation needs StartTime = 0 and EndTime to be the original end time for proper offsets
	FWaveTransformUObjectConfiguration InConfiguration = InOutConfiguration;
	InConfiguration.StartTime = 0.f;
	InConfiguration.EndTime = InOutConfiguration.GetOriginalNumFrames() / InOutConfiguration.SampleRate;

	FadeTransformation->UpdateConfiguration(InConfiguration);
}

const FTransformationFadeFunctionData& UWaveformTransformationTrimFade::GetFadeRegion() const
{
	check(FadeTransformation);
	check(FadeTransformation->GetFadeRegionsNum() == 1);

	return FadeTransformation->GetFadeRegions()[0];
}

FTransformationFadeFunctionData& UWaveformTransformationTrimFade::GetMutableFadeRegion() const
{
	check(FadeTransformation);
	check(FadeTransformation->GetFadeRegionsNum() == 1);

	return FadeTransformation->GetMutableFadeRegions()[0];
}

void UWaveformTransformationTrimFade::UpdateDurationProperties(const float InAvailableDuration)
{
	check(InAvailableDuration > 0.f);
	AvailableWaveformDuration = InAvailableDuration;
	StartTime = FMath::Clamp(StartTime, 0.f, AvailableWaveformDuration - UE_KINDA_SMALL_NUMBER);
	EndTime = EndTime < 0 ? EndTime = AvailableWaveformDuration : FMath::Clamp(EndTime, StartTime + UE_KINDA_SMALL_NUMBER, AvailableWaveformDuration);
}

const double UFadeFunctionLinear::GetFadeInCurveValue(const double FadeFraction) const
{
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return FadeFraction;
}

const double UFadeFunctionLinear::GetFadeOutCurveValue(const double FadeFraction) const
{
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return 1.0 - FadeFraction;
}

const double UFadeCurveFunctionExponential::GetFadeInCurveValue(const double FadeFraction) const
{
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return FMath::Pow(FadeFraction, FadeCurve);
}

const double UFadeCurveFunctionExponential::GetFadeOutCurveValue(const double FadeFraction) const
{
	// Pow functions return NaN for negative bases with non-integer exponents
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return FMath::Pow(-FadeFraction + 1.0, FadeCurve);
}

const double UFadeCurveFunctionLogarithmic::GetFadeInCurveValue(const double FadeFraction) const
{
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return FMath::Pow(FadeFraction, FadeCurve);
}

const double UFadeCurveFunctionLogarithmic::GetFadeOutCurveValue(const double FadeFraction) const
{
	// Pow functions return NaN for negative bases with non-integer exponents
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);

	return FMath::Pow(-FadeFraction + 1.0, FadeCurve);
}

const double UFadeCurveFunctionSigmoid::GetFadeInCurveValue(const double FadeFraction) const
{
	// LogX will produce NaN if FadeFraction is <= 0 or is 1
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);
	check(WaveformTrasnformationTrimFadePrivate::XMultiplier > 0.0);

	if (FadeFraction == 0.f || FadeFraction == 1.f)
	{
		return FadeFraction;
	}

	double CurveValue = 1;
	const double x = FadeFraction;
	const double a = static_cast<float>(-FMath::Clamp(SFadeCurve, 0.1f, 1.0f));
	const double k = 20 * FMath::Pow(a, 5);

	if (FadeFraction <= 0.5)
	{
		CurveValue = -(FMath::Tanh(k * PI * FMath::LogX(10, WaveformTrasnformationTrimFadePrivate::XMultiplier * x) + ((k * PI) / 2))) / 2 + 0.5;
	}
	else
	{
		CurveValue = (FMath::Tanh(k * PI * FMath::LogX(10, WaveformTrasnformationTrimFadePrivate::XMultiplier * (-x + 1)) + ((k * PI) / 2))) / 2 + 0.5;
	}

	return CurveValue;
}

const double UFadeCurveFunctionSigmoid::GetFadeOutCurveValue(const double FadeFraction) const
{
	// LogX will produce NaN if FadeFraction is <= 0 or is 1
	check(FadeFraction <= 1.f);
	check(FadeFraction >= 0.f);
	check(WaveformTrasnformationTrimFadePrivate::XMultiplier > 0.0);

	if (FadeFraction == 0.f || FadeFraction == 1.f)
	{
		return 1.0 - FadeFraction;
	}

	double CurveValue = 1;
	const double x = FadeFraction;
	const double a = static_cast<float>(FMath::Clamp(SFadeCurve, 0.1f, 1.0f));
	const double k = 20 * FMath::Pow(a, 5);

	if (FadeFraction <= 0.5)
	{
		CurveValue = -(FMath::Tanh(k * PI * FMath::LogX(10, WaveformTrasnformationTrimFadePrivate::XMultiplier * x) + ((k * PI) / 2))) / 2 + 0.5;
	}
	else
	{
		CurveValue = (FMath::Tanh(k * PI * FMath::LogX(10, WaveformTrasnformationTrimFadePrivate::XMultiplier * (-x + 1)) + ((k * PI) / 2))) / 2 + 0.5;
	}

	return CurveValue;
}

void FFadeFunctionData::MigrateToTransformationFadeFunction(const TObjectPtr<UWaveformTransformationFade> InFadeTransformation, FTransformationFadeFunctionData& NewFadeRegion)
{
	check(InFadeTransformation);
	check(InFadeTransformation->GetFadeRegionsNum() == 1);
	
	if(FadeIn)
	{
		WaveformTrasnformationTrimFadePrivate::MigrateToTransformationFadeFunction(InFadeTransformation, FadeIn, NewFadeRegion.FadeIn);

		FadeIn = nullptr;
	}

	if (FadeOut)
	{
		WaveformTrasnformationTrimFadePrivate::MigrateToTransformationFadeFunction(InFadeTransformation, FadeOut, NewFadeRegion.FadeOut);

		FadeOut = nullptr;
	}
}
