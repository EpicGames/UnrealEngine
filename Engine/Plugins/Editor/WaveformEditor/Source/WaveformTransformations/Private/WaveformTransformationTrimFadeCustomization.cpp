// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationTrimFadeCustomization.h"

#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "IPropertyUtilities.h"
#include "PropertyHandle.h"
#include "WaveformTransformationTrimFade.h"

TSharedRef<IPropertyTypeCustomization> FWaveformTransformationTrimFadeCustomization::MakeInstance()
{
	return MakeShareable(new FWaveformTransformationTrimFadeCustomization);
}

void FWaveformTransformationTrimFadeCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle,
	FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	constexpr bool bDisplayDefaultPropertyButtons = false;

	HeaderRow.NameContent()[PropertyHandle->CreatePropertyNameWidget()]
	.ValueContent()[PropertyHandle->CreatePropertyValueWidget(bDisplayDefaultPropertyButtons)];
}

void FWaveformTransformationTrimFadeCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle,
	IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& CustomizationUtils)
{
	IDetailGroup& TrimGroup = ChildBuilder.AddGroup(TEXT("TrimGroup"), FText::FromString(TEXT("Trim")), true);

	TSharedPtr<IPropertyHandle> StartTimeHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UWaveformTransformationTrimFade, StartTime));
	if (StartTimeHandle && StartTimeHandle->IsValidHandle())
	{
		TrimGroup.AddPropertyRow(StartTimeHandle.ToSharedRef());
	}

	TSharedPtr<IPropertyHandle> EndTimeHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UWaveformTransformationTrimFade, EndTime));
	if (EndTimeHandle && EndTimeHandle->IsValidHandle())
	{
		TrimGroup.AddPropertyRow(EndTimeHandle.ToSharedRef());
	}

	IDetailGroup& FadeGroup = ChildBuilder.AddGroup(TEXT("FadeGroup"), FText::FromString(TEXT("Fade")), true);

	TSharedPtr<IPropertyHandle> FadeTransformationHandle = PropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UWaveformTransformationTrimFade, FadeTransformation));
	if (FadeTransformationHandle && FadeTransformationHandle->IsValidHandle())
	{
		TSharedPtr<IPropertyHandle> FadeRegionArrayHandle = FadeTransformationHandle->GetChildHandle(UWaveformTransformationFade::GetFadeRegionsPropertyName());
		if (FadeRegionArrayHandle && FadeRegionArrayHandle->IsValidHandle())
		{
			TSharedPtr<IPropertyHandleArray> FadeRegionArray = FadeRegionArrayHandle->AsArray();
			if (FadeRegionArray)
			{
				uint32 NumRegions = 0;
				FadeRegionArray->GetNumElements(NumRegions);

				for (uint32 Index = 0; Index < NumRegions; Index++)
				{
					TSharedPtr<IPropertyHandle> FadeRegionHandle = FadeRegionArray->GetElement(Index);
					if (FadeRegionHandle && FadeRegionHandle->IsValidHandle())
					{
						// Only show the Fade In and Out properties necessary for TrimFades
						TSharedPtr<IPropertyHandle> FadeInHandle = FadeRegionHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTransformationFadeFunctionData, FadeIn));
						if (FadeInHandle && FadeInHandle->IsValidHandle())
						{
							IDetailGroup& FadeInGroup = FadeGroup.AddGroup(TEXT("FadeInGroup"), FText::FromString(TEXT("Fade In")), true);
							FadeInGroup.AddPropertyRow(FadeInHandle.ToSharedRef());

							TSharedPtr<IPropertyHandle> FadeInDurHandle = FadeInHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UTransformationFadeFunction, Duration));
							if (FadeInDurHandle && FadeInDurHandle->IsValidHandle())
							{
								FadeInGroup.AddPropertyRow(FadeInDurHandle.ToSharedRef());
							}

							TSharedPtr<IPropertyHandle> FadeInOffsetHandle = FadeInHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UTransformationFadeFunction, FrameOffset));
							if (FadeInOffsetHandle && FadeInOffsetHandle->IsValidHandle())
							{
								FadeInOffsetHandle->MarkHiddenByCustomization();
							}

							UObject* FadeInObject = nullptr;
							if (FadeInHandle->GetValue(FadeInObject) == FPropertyAccess::Success && FadeInObject)
							{
								if (FadeInObject->IsA<UTransformationFadeCurveFunctionExponential>())
								{
									TSharedPtr<IPropertyHandle> FadeCurveHandle = FadeInHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UTransformationFadeCurveFunctionExponential, FadeCurve));
									if (FadeCurveHandle && FadeCurveHandle->IsValidHandle())
									{
										FadeInGroup.AddPropertyRow(FadeCurveHandle.ToSharedRef());
									}
								}
								else if (FadeInObject->IsA<UTransformationFadeCurveFunctionLogarithmic>())
								{
									TSharedPtr<IPropertyHandle> FadeCurveHandle = FadeInHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UTransformationFadeCurveFunctionLogarithmic, FadeCurve));
									if (FadeCurveHandle && FadeCurveHandle->IsValidHandle())
									{
										FadeInGroup.AddPropertyRow(FadeCurveHandle.ToSharedRef());
									}
								}
								else if (FadeInObject->IsA<UTransformationFadeCurveFunctionSigmoid>())
								{
									TSharedPtr<IPropertyHandle> FadeCurveHandle = FadeInHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UTransformationFadeCurveFunctionSigmoid, SigmoidFadeCurve));
									if (FadeCurveHandle && FadeCurveHandle->IsValidHandle())
									{
										FadeInGroup.AddPropertyRow(FadeCurveHandle.ToSharedRef());
									}
								}
							}
						}

						TSharedPtr<IPropertyHandle> FadeOutHandle = FadeRegionHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FTransformationFadeFunctionData, FadeOut));
						if (FadeOutHandle && FadeOutHandle->IsValidHandle())
						{
							IDetailGroup& FadeOutGroup = FadeGroup.AddGroup(TEXT("FadeOutGroup"), FText::FromString(TEXT("Fade Out")), true);
							FadeOutGroup.AddPropertyRow(FadeOutHandle.ToSharedRef());

							TSharedPtr<IPropertyHandle> FadeInDurHandle = FadeOutHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UTransformationFadeFunction, Duration));
							if (FadeInDurHandle && FadeInDurHandle->IsValidHandle())
							{
								FadeOutGroup.AddPropertyRow(FadeInDurHandle.ToSharedRef());
							}

							TSharedPtr<IPropertyHandle> FadeOutOffsetHandle = FadeOutHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UTransformationFadeFunction, FrameOffset));
							if (FadeOutOffsetHandle && FadeOutOffsetHandle->IsValidHandle())
							{
								FadeOutOffsetHandle->MarkHiddenByCustomization();
							}

							UObject* FadeOutObject = nullptr;
							if (FadeOutHandle->GetValue(FadeOutObject) == FPropertyAccess::Success && FadeOutObject)
							{
								if (FadeOutObject->IsA<UTransformationFadeCurveFunctionExponential>())
								{
									TSharedPtr<IPropertyHandle> FadeCurveHandle = FadeOutHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UTransformationFadeCurveFunctionExponential, FadeCurve));
									if (FadeCurveHandle && FadeCurveHandle->IsValidHandle())
									{
										FadeOutGroup.AddPropertyRow(FadeCurveHandle.ToSharedRef());
									}
								}
								else if (FadeOutObject->IsA<UTransformationFadeCurveFunctionLogarithmic>())
								{
									TSharedPtr<IPropertyHandle> FadeCurveHandle = FadeOutHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UTransformationFadeCurveFunctionLogarithmic, FadeCurve));
									if (FadeCurveHandle && FadeCurveHandle->IsValidHandle())
									{
										FadeOutGroup.AddPropertyRow(FadeCurveHandle.ToSharedRef());
									}
								}
								else if (FadeOutObject->IsA<UTransformationFadeCurveFunctionSigmoid>())
								{
									TSharedPtr<IPropertyHandle> FadeCurveHandle = FadeOutHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(UTransformationFadeCurveFunctionSigmoid, SigmoidFadeCurve));
									if (FadeCurveHandle && FadeCurveHandle->IsValidHandle())
									{
										FadeOutGroup.AddPropertyRow(FadeCurveHandle.ToSharedRef());
									}
								}
							}
						}
					}
				}
			}
		}

	}

}