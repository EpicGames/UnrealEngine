// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IPropertyTypeCustomization.h"

/**
 * Using IPropertyTypeCustomization to edit the details view of the TrimFade transformation and remove unneeded information in the waveform editor
 */
class FWaveformTransformationTrimFadeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> PropertyHandle, FDetailWidgetRow& HeaderRow,
								 IPropertyTypeCustomizationUtils& CustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> PropertyHandle, IDetailChildrenBuilder& ChildBuilder,
								   IPropertyTypeCustomizationUtils& CustomizationUtils) override;
};
