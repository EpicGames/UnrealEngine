// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationsModule.h"

#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "WaveformTransformationLog.h"
#include "WaveformTransformationFadeCustomization.h"
#include "WaveformTransformationMarkersObjectCustomization.h"
#include "WaveformTransformationTrimFadeCustomization.h"

DEFINE_LOG_CATEGORY(LogWaveformTransformation);

void FWaveformTransformationsModule::StartupModule()
{
	IModuleInterface::StartupModule();

	FPropertyEditorModule* PropertyEditorModule = FModuleManager::LoadModulePtr<FPropertyEditorModule>("PropertyEditor");

	if (PropertyEditorModule == nullptr)
	{
		return;
	}

	PropertyEditorModule->RegisterCustomPropertyTypeLayout(
		"WaveformTransformationMarkers",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWaveformTransformationMarkersObjectCustomization::MakeInstance)
	);

	PropertyEditorModule->RegisterCustomPropertyTypeLayout(
		"WaveformTransformationTrimFade",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWaveformTransformationTrimFadeCustomization::MakeInstance)
	);

	PropertyEditorModule->RegisterCustomPropertyTypeLayout(
		"WaveformTransformationFade",
		FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FWaveformTransformationFadeCustomization::MakeInstance)
	);

	PropertyEditorModule->NotifyCustomizationModuleChanged();
}

void FWaveformTransformationsModule::ShutdownModule()
{
	IModuleInterface::ShutdownModule();

	if (FModuleManager::Get().IsModuleLoaded("PropertyEditor"))
	{
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("WaveformTransformationMarkers");
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("WaveformTransformationTrimFade");
		PropertyEditorModule.UnregisterCustomPropertyTypeLayout("WaveformTransformationFade");
		PropertyEditorModule.NotifyCustomizationModuleChanged();
	}
}

IMPLEMENT_MODULE(FWaveformTransformationsModule, WaveformTransformations);