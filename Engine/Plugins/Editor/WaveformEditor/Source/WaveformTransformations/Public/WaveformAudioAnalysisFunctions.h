// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DSP/AlignedBuffer.h"

#define UE_API WAVEFORMTRANSFORMATIONS_API

namespace WaveformAudioAnalysis
{
	UE_API float GetRMSPeak(const Audio::FAlignedFloatBuffer& InputAudio, float SampleRate, int32 NumChannels);

	UE_API float GetLoudnessPeak(const Audio::FAlignedFloatBuffer& InputAudio, float SampleRate, int32 NumChannels);

	UE_API float GetPeakSampleValue(const Audio::FAlignedFloatBuffer& InputAudio);

	// Gated Loudness K-Weighted relative to Full Scale (Not-Real-Time)
	UE_API float GetLUFS(const Audio::FAlignedFloatBuffer& InputAudio, float SampleRate, int32 NumChannels);
}

#undef UE_API