// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Factories/SoundSourceEffectFactory.h"
#include "IWaveformTransformation.h"
#include "UObject/Object.h"
#include "UObject/StrongObjectPtr.h"
#include "WaveformTransformationEffectChain.generated.h"

#define UE_API WAVEFORMTRANSFORMATIONS_API

class UE_DEPRECATED(5.8, "WaveTransformationEffectChain has been deprecated") FWaveTransformationEffectChain : public Audio::IWaveTransformation
{
public:
	UE_API FWaveTransformationEffectChain(TArray<TObjectPtr<class USoundEffectSourcePreset>>& InEffectPresets);

	UE_API virtual void ProcessAudio(Audio::FWaveformTransformationWaveInfo& InOutWaveInfo) const override;

	virtual bool SupportsRealtimePreview() const override { return true; }
	virtual constexpr Audio::ETransformationPriority FileChangeLengthPriority() const override { return Audio::ETransformationPriority::High; }

private:
	TArray<TStrongObjectPtr<class USoundEffectSourcePreset>> Presets;
};

// USoundEffectPresets need diff tag hashes ONLY IF linked to waveform transformations.
// Removing their connection to waveform transformations is sufficient to avoid the need for cook diff tag support for USoundEffectSourcePresets.
// This done by deprecating WaveformTransformationEffectChain.
UCLASS(MinimalAPI, hidden)
class UE_DEPRECATED(5.8, "WaveformTransformationEffectChain has been deprecated") UWaveformTransformationEffectChain : public UWaveformTransformationBase
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Trim")
	TObjectPtr<class USoundEffectSourcePresetChain> EffectChain;

	UPROPERTY(EditAnywhere, Instanced, Category = "Trim")
	TArray<TObjectPtr<class USoundEffectSourcePreset>> InlineEffects;
	
public:
	
	UE_API virtual Audio::FTransformationPtr CreateTransformation() const override;
	virtual FString GetTransformationHash() const override { return ""; }
};

#undef UE_API
