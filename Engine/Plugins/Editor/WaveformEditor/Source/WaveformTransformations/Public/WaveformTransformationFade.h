// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "IWaveformTransformation.h"

#include "WaveformTransformationFade.generated.h"

#define UE_API WAVEFORMTRANSFORMATIONS_API

UCLASS(MinimalAPI, Abstract, EditInlineNew, DefaultToInstanced, CollapseCategories, NotBlueprintable)
class UTransformationFadeFunction : public UObject
{
	GENERATED_BODY()
public:

	virtual const double GetFadeInCurveValue(const double FadeFraction) const { return FadeFraction; }
	virtual const double GetFadeOutCurveValue(const double FadeFraction) const { return 1.0 - FadeFraction; }

	UPROPERTY(EditAnywhere, Category = "Fade", BlueprintReadWrite, meta = (ClampMin = 0.0, DisplayName = "Fade Duration"))
	float Duration = 0.5f;

	UPROPERTY(EditAnywhere, Category = "Fade", BlueprintReadWrite, meta = (ClampMin = 0.0, DisplayName = "Fade Frame Offset"))
	int64 FrameOffset = 0;
};

UCLASS(MinimalAPI, Abstract, EditInlineNew, DefaultToInstanced, CollapseCategories, NotBlueprintable)
class UTransformationFadeCurveFunction : public UTransformationFadeFunction
{
	GENERATED_BODY()
public:

	virtual const double GetFadeInCurveValue(const double FadeFraction) const override { return FadeFraction; }
	virtual const double GetFadeOutCurveValue(const double FadeFraction) const override { return 1.0 - FadeFraction; }

	virtual void SetFadeCurve(const float InFadeCurve) {}
	virtual const float GetFadeCurve() const { return 0.f; }
	virtual const FText GetFadeCurvePropertyName() const { return FText::FromName(NAME_None); }

protected:
	virtual const float GetFadeCurveMin() const { return 0.f; }
	virtual const float GetFadeCurveMax() const { return 1.f; }
};

UCLASS(MinimalAPI, NotBlueprintable, meta = (DisplayName = "Linear"))
class UTransformationFadeFunctionLinear : public UTransformationFadeFunction
{
	GENERATED_BODY()
public:
	
	UE_API virtual const double GetFadeInCurveValue(const double FadeFraction) const override;
	UE_API virtual const double GetFadeOutCurveValue(const double FadeFraction) const override;
};

UCLASS(MinimalAPI, NotBlueprintable, meta = (DisplayName = "Exponential"))
class UTransformationFadeCurveFunctionExponential : public UTransformationFadeCurveFunction
{
	GENERATED_BODY()
public:

	UE_API virtual const double GetFadeInCurveValue(const double FadeFraction) const override;
	UE_API virtual const double GetFadeOutCurveValue(const double FadeFraction) const override;

	virtual void SetFadeCurve(const float InFadeCurve) override { FadeCurve = FMath::Clamp(InFadeCurve, GetFadeCurveMin(), GetFadeCurveMax()); }
	virtual const float GetFadeCurve() const override { return FadeCurve; }
	virtual const FText GetFadeCurvePropertyName() const override { return FText::FromName(GET_MEMBER_NAME_CHECKED(UTransformationFadeCurveFunctionExponential, FadeCurve)); }

	UPROPERTY(EditAnywhere, Category = "Fade", meta = (ClampMin = 1, ClampMax = 10.0, DisplayName = "Fade Curve"))
	float FadeCurve = 3.f;

protected:
	virtual const float GetFadeCurveMin() const override { return 1.f; }
	virtual const float GetFadeCurveMax() const override { return 10.f; }
};

UCLASS(MinimalAPI, NotBlueprintable, meta = (DisplayName = "Logarithmic"))
class UTransformationFadeCurveFunctionLogarithmic : public UTransformationFadeCurveFunction
{
	GENERATED_BODY()
public:

	UE_API virtual const double GetFadeInCurveValue(const double FadeFraction) const override;
	UE_API virtual const double GetFadeOutCurveValue(const double FadeFraction) const override;

	virtual void SetFadeCurve(const float InFadeCurve) override { FadeCurve = FMath::Clamp(InFadeCurve, GetFadeCurveMin(), GetFadeCurveMax()); }
	virtual const float GetFadeCurve() const override { return FadeCurve; }
	virtual const FText GetFadeCurvePropertyName() const override { return FText::FromName(GET_MEMBER_NAME_CHECKED(UTransformationFadeCurveFunctionLogarithmic, FadeCurve)); }

	UPROPERTY(EditAnywhere, Category = "Fade", meta = (ClampMin = 0.0, ClampMax = 1, DisplayName = "Fade Curve"))
	float FadeCurve = 0.25f;
};

UCLASS(MinimalAPI, NotBlueprintable, meta = (DisplayName = "Sigmoid"))
class UTransformationFadeCurveFunctionSigmoid : public UTransformationFadeCurveFunction
{
	GENERATED_BODY()
public:

	UE_API virtual const double GetFadeInCurveValue(const double FadeFraction) const override;
	UE_API virtual const double GetFadeOutCurveValue(const double FadeFraction) const override;

	virtual void SetFadeCurve(const float InFadeCurve) override { SigmoidFadeCurve = FMath::Clamp(InFadeCurve, GetFadeCurveMin(), GetFadeCurveMax()); }
	virtual const float GetFadeCurve() const override { return SigmoidFadeCurve; }
	virtual const FText GetFadeCurvePropertyName() const override { return FText::FromName(GET_MEMBER_NAME_CHECKED(UTransformationFadeCurveFunctionSigmoid, SigmoidFadeCurve)); }

	UPROPERTY(EditAnywhere, Category = "Fade", meta = (ClampMin = 0.0, ClampMax = 1, DisplayName = "S-Fade Curve"))
	float SigmoidFadeCurve = 0.6f;
};

USTRUCT()
struct FTransformationFadeFunctionData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Fade", Instanced, meta = (DisplayName = "Fade In Type"))
	TObjectPtr<UTransformationFadeFunction> FadeIn;

	UPROPERTY(EditAnywhere, Category = "Fade", Instanced, meta = (DisplayName = "Fade Out Type"))
	TObjectPtr<UTransformationFadeFunction> FadeOut;

	UPROPERTY(EditAnywhere, Category = "Fade", meta = (DisplayName = "Link Fade Durations"))
	bool bLinkDurations = false;
};

UENUM()
enum class EWaveEditorTransformationFadeMode : uint8
{
	Linear = 0,
	Exponential,
	Logarithmic,
	Sigmoid
};

class FWaveTransformationFade : public Audio::IWaveTransformation
{
public:
	UE_API explicit FWaveTransformationFade(const TArray<FTransformationFadeFunctionData>& InFadeRegions, const int64 InStartFrameOffset, const int64 InEndFrameOffset);
	UE_API virtual void ProcessAudio(Audio::FWaveformTransformationWaveInfo& InOutWaveInfo) const override;

private:

	TArray<FTransformationFadeFunctionData> FadeRegions;
	int64 StartFrameOffset = 0;
	int64 EndFrameOffset = 0;
};

// Maintains an array of fade regions that can be placed in any location on the waveform.
// Each region has a start and end fade with optional offsets from the start and end of the waveform
// This transformation can be used in other transformation (ex. TrimFade)
UCLASS(MinimalAPI)
class UWaveformTransformationFade final : public UWaveformTransformationBase
{
	GENERATED_BODY()
public:

	UWaveformTransformationFade(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UE_API virtual FString GetTransformationHash() const override;
#endif // WITH_EDITOR;

	static UE_API const TMap<EWaveEditorTransformationFadeMode, TSubclassOf<UTransformationFadeFunction>> FadeModeToFadeFunctionMap;

	UE_API virtual Audio::FTransformationPtr CreateTransformation() const override;

	UE_API void UpdateConfiguration(FWaveTransformUObjectConfiguration& InOutConfiguration) override;

	const float GetSampleRate() const { return SampleRate; }

	const int32 GetFadeRegionsNum() const { return FadeRegions.Num(); }
	const TArray<FTransformationFadeFunctionData>& GetFadeRegions() const { return FadeRegions; }
	TArray<FTransformationFadeFunctionData>& GetMutableFadeRegions() { return FadeRegions; }

	// Remove all fade regions other than the first one.
	// Logs a warning if it removes any fade regions
	UE_API void LimitFadeRegionsToOne();

	static FName GetFadeRegionsPropertyName()
	{
		return GET_MEMBER_NAME_CHECKED(UWaveformTransformationFade, FadeRegions);
	}

protected:
	UPROPERTY(EditAnywhere, Category = "Fade", meta = (DisplayName = "Fade Regions"))
	TArray<FTransformationFadeFunctionData> FadeRegions;

	
private:	

	UE_API void UpdateDurationProperties(const float InAvailableDuration);
	float AvailableWaveformDuration = -1.f;
	float SampleRate = 0.f;

	int64 StartFrameOffset = 0;
	int64 EndFrameOffset = 0;
	int64 OriginalNumFrames = 0;
};

#undef UE_API
