// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API WAVEFORMTRANSFORMATIONS_API

UE_API DECLARE_LOG_CATEGORY_EXTERN(LogWaveformTransformation, Log, All);

#undef UE_API