// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWaveformTransformation.h"
#include "InputCore.h"

#include "WaveformTransformationMarkers.generated.h"

#define UE_API WAVEFORMTRANSFORMATIONS_API

class USoundWave;

enum class ELoopModificationControls : uint8
{
	None = 0,
	LeftHandleIncrement,
	LeftHandleDecrement,
	RightHandleIncrement,
	RightHandleDecrement,
	IncreaseIncrement,
	DecreaseIncrement,
	SelectNextLoop,
	SelectPreviousLoop,
	LeftHandleNextZeroCrossing,
	LeftHandlePreviousZeroCrossing,
	RightHandleNextZeroCrossing,
	RightHandlePreviousZeroCrossing,
};

//Used to make CuePoints work with PropertyHandles
UCLASS(MinimalAPI, BlueprintType, DefaultToInstanced)
class UWaveCueArray : public UObject
{
	GENERATED_BODY()
public:
#if WITH_EDITOR
	UE_API void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UE_API void PostEditChangeChainProperty(struct FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	//If uninitialized, init Marker array
	UE_API void InitMarkersIfNotSet(const TArray<FSoundWaveCuePoint>& InMarkers);
	//Uninitialize and empty Marker array
	UE_API void Reset();

	UE_API void EnableLoopRegion(FSoundWaveCuePoint* OutSoundWaveCue, bool bSetLoopRegion = true);

	UPROPERTY(EditAnywhere, Category = "Markers")
	TArray<FSoundWaveCuePoint> CuesAndLoops;
	int32 SelectedCue = INDEX_NONE;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.7, "ModifyMarkerLoopRegionDelegate has been deprecated.")
	DECLARE_DELEGATE_OneParam(ModifyMarkerLoopRegionDelegate, ELoopModificationControls);
	UE_DEPRECATED(5.7, "ModifyMarkerLoop has been deprecated.")
	ModifyMarkerLoopRegionDelegate ModifyMarkerLoop;

	UE_DEPRECATED(5.7, "CycleMarkerLoopRegionDelegate has been deprecated.")
	DECLARE_DELEGATE_OneParam(CycleMarkerLoopRegionDelegate, ELoopModificationControls);
	UE_DEPRECATED(5.7, "CycleMarkerLoop has been deprecated.")
	CycleMarkerLoopRegionDelegate CycleMarkerLoop;

	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// To minimize complexity while supporting all common editing cases, loops have a min length of 10 frames
	static constexpr int64 MinLoopSize = 10;

	DECLARE_DELEGATE(OnCueChange);
	OnCueChange CueChanged;

private:
	UPROPERTY()
	bool bIsInitialized = false;
};

class FWaveTransformationMarkers : public Audio::IWaveTransformation
{
public:
	UE_API explicit FWaveTransformationMarkers(double InStartLoopTime, double InEndLoopTime);
	UE_API virtual void ProcessAudio(Audio::FWaveformTransformationWaveInfo& InOutWaveInfo) const override;

	virtual constexpr Audio::ETransformationPriority FileChangeLengthPriority() const override { return Audio::ETransformationPriority::Low; }

private:
	double StartLoopTime = 0.0;
	double EndLoopTime = 0.0;
};

UCLASS(MinimalAPI)
class UWaveformTransformationMarkers final : public UWaveformTransformationBase
{
	GENERATED_BODY()
public:
	UE_API UWaveformTransformationMarkers(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	
	virtual void PostEditUndo() override;
#endif // WITH_EDITOR

	UPROPERTY(VisibleAnywhere, Category = "Markers")
	TObjectPtr<UWaveCueArray> Markers;

	// These properties are hidden in editor by an IPropertyTypeCustomization
	UPROPERTY(EditAnywhere, Category = "Loop Preview", meta = (ClampMin = 0.0))
	double StartLoopTime = 0.0;
	// When EndLoopTime is < 0 it skips processing the audio in FWaveTransformationMarkers::ProcessAudio 
	UPROPERTY(EditAnywhere, Category = "Loop Preview")
	double EndLoopTime = -1.0;
protected:
	UPROPERTY(EditAnywhere, Category = "Loop Preview")
	bool bIsPreviewingLoopRegion = false;
public:
	UE_API virtual Audio::FTransformationPtr CreateTransformation() const override;

	UE_API void UpdateConfiguration(FWaveTransformUObjectConfiguration& InOutConfiguration) override;
	UE_API void OverwriteTransformation() override;

	UE_DEPRECATED(5.7, "ModifyMarkerLoopRegion const has been deprecated, use the non-const version")
	UE_API void ModifyMarkerLoopRegion(ELoopModificationControls Modification) const;
	UE_DEPRECATED(5.7, "CycleMarkerLoopRegion const has been deprecated, use the non-const version")
	UE_API void CycleMarkerLoopRegion(ELoopModificationControls Modification) const;

	// Shift marker and loop region bounds with toolkit commands bound in the waveform editor
	UE_API void ModifyMarkerLoopRegion(ELoopModificationControls Modification, const TArray<float> &PCMArray, float NoiseThreshold = 0.01f);
	// Set SelectedCue by cycling through cues and loops with toolkit commands bound in the waveform editor
	UE_API void CycleMarkerLoopRegion(ELoopModificationControls Modification);

	static UE_API int64 GetFramesToNextZeroCrossing(int64 StartFrame, int64 EndFrame, const TArray<float>& PCMArray, bool bCheckIncrementing, float NoiseThreshold, int ChannelCount);
	static UE_API int64 CheckZeroCrossingFrame(int64 CurrentSample, const TArray<float>& PCMArray, bool& bAllSamplesPositiveLastHighNoise, bool& bAllSamplesNegativeLastHighNoise, float NoiseThreshold, int ChannelCount);

	UE_API bool IsLoopRegionActive() const;
	UE_API void SetLoopPreviewing();

	// Returns true if reset
	UE_API bool ResetLoopPreviewing();
	// Returns true if reset
	UE_API bool AdjustLoopPreviewIfNotAligned();

#if WITH_EDITOR
	UE_API virtual void OverwriteSoundWaveData(USoundWave& InOutSoundWave) override;
	UE_API virtual void GetTransformationInfo(FWaveformTransformationInfo& InOutTransformationInfo) const override;
	UE_API virtual FString GetTransformationHash() const override;
#endif //WITH_EDITOR

	UE_API FSoundWaveCuePoint* GetSelectedMarker() const;

private:
	
	UE_API void PreviewSelectedLoop();
	UE_API void SetIsPreviewingLoopRegion(double InStartTime, double InEndTime, bool bIsPreviewing);
	UE_API void SelectLoopRegionByKeyboard(const FKey& PressedKey);
	
	int32 SelectedIncrement = 2;

	float SampleRate = 0.f;
	float AvailableWaveformDuration = -1.f;
	int32 NumChannels = 0;
	int64 NumAvailableSamples = 0;

	// Made a UPROPERTY so that it is captured when the soundwave is duplicated during export
	UPROPERTY()
	int64 StartFrameOffset = 0;

	bool bCachedIsPreviewingLoopRegion = false;
	bool bCachedSoundWaveLoopState = false;

	bool bHasLoggedCutByLoop = false;
};

#undef UE_API
