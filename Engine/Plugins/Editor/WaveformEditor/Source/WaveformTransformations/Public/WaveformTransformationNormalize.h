// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWaveformTransformation.h"
#include "UObject/Object.h"
#include "WaveformTransformationNormalize.generated.h"

#define UE_API WAVEFORMTRANSFORMATIONS_API

UENUM()
enum class ENormalizationMode : uint8
{
	Peak,
	RMS,
	DWeightedLoudness,
	COUNT UMETA(Hidden)
};

class FWaveTransformationNormalize : public Audio::IWaveTransformation
{
public:
	UE_API FWaveTransformationNormalize(float InTarget, float InMaxGain, ENormalizationMode InMode);

	UE_API virtual void ProcessAudio(Audio::FWaveformTransformationWaveInfo& InOutWaveInfo) const override;

private:
	float Target = 0.f;
	float MaxGain = 0.f;
	ENormalizationMode Mode = ENormalizationMode::Peak;
};

UCLASS(MinimalAPI)
class UWaveformTransformationNormalize final : public UWaveformTransformationBase
{
	GENERATED_BODY()

	// Target maximum volume for this soundwave, in decibels
	UPROPERTY(EditAnywhere, Category="Normalization", meta=(ClampMax = 0.0))
	float Target = 0.f;

	// Will not apply more gain than this, even if the sound is very quiet
	UPROPERTY(EditAnywhere, Category="Normalization", meta=(ClampMin = 0.0))
	float MaxGain = 0.f;

	// what type of analysis to run to find the peak value
	UPROPERTY(EditAnywhere, Category="Normalization")
	ENormalizationMode Mode;
	
public:
	UE_API virtual Audio::FTransformationPtr CreateTransformation() const override;
	virtual constexpr Audio::ETransformationPriority GetTransformationPriority() const { return Audio::ETransformationPriority::Low; }

#if WITH_EDITOR
	UE_API virtual FString GetTransformationHash() const override;
#endif // WITH_EDITOR

};

#undef UE_API
