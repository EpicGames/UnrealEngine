// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "IWaveformTransformation.h"
#include "WaveformTransformationFade.h"

#include "WaveformTransformationTrimFade.generated.h"

#define UE_API WAVEFORMTRANSFORMATIONS_API

UCLASS(MinimalAPI, Abstract, EditInlineNew, DefaultToInstanced, CollapseCategories, NotBlueprintable)
class UE_DEPRECATED(5.8, "UFadeFunction has been deprecated. Use UTransformationFadeFunction.") UFadeFunction : public UObject
{
	GENERATED_BODY()
public:

	virtual const double GetFadeInCurveValue(const double FadeFraction) const { return FadeFraction; }
	virtual const double GetFadeOutCurveValue(const double FadeFraction) const { return FadeFraction; }

	UPROPERTY(EditAnywhere, Category = "Fade", BlueprintReadWrite, meta = (ClampMin = 0.0, DisplayName = "Fade Duration"))
	float Duration = 0.5f;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UCLASS(MinimalAPI, Abstract, EditInlineNew, DefaultToInstanced, CollapseCategories, NotBlueprintable)
class UE_DEPRECATED(5.8, "UFadeFunction has been deprecated. Use UTransformationFadeFunction.") UFadeCurveFunction : public UFadeFunction
{
	GENERATED_BODY()
public:

	virtual const double GetFadeInCurveValue(const double FadeFraction) const override { return FadeFraction; }
	virtual const double GetFadeOutCurveValue(const double FadeFraction) const override { return FadeFraction; }

	virtual void SetFadeCurve(const float InFadeCurve) {}
	virtual const float GetFadeCurve() const { return 0.f; }
	virtual const FText GetFadeCurvePropertyName() const { return FText::FromName(NAME_None); }

protected:
	virtual const float GetFadeCurveMin() const { return 0.f; }
	virtual const float GetFadeCurveMax() const { return 1.f; }
};

UCLASS(MinimalAPI, NotBlueprintable, meta = (DisplayName = "Linear"))
class UE_DEPRECATED(5.8, "UFadeFunction has been deprecated. Use UTransformationFadeFunction.") UFadeFunctionLinear : public UFadeFunction
{
	GENERATED_BODY()
public:
	
	UE_API virtual const double GetFadeInCurveValue(const double FadeFraction) const override;
	UE_API virtual const double GetFadeOutCurveValue(const double FadeFraction) const override;
};

UCLASS(MinimalAPI, NotBlueprintable, meta = (DisplayName = "Exponential"))
class UE_DEPRECATED(5.8, "UFadeFunction has been deprecated. Use UTransformationFadeFunction.") UFadeCurveFunctionExponential : public UFadeCurveFunction
{
	GENERATED_BODY()
public:

	UE_API virtual const double GetFadeInCurveValue(const double FadeFraction) const override;
	UE_API virtual const double GetFadeOutCurveValue(const double FadeFraction) const override;

	virtual void SetFadeCurve(const float InFadeCurve) override { FadeCurve = FMath::Clamp(InFadeCurve, GetFadeCurveMin(), GetFadeCurveMax()); }
	virtual const float GetFadeCurve() const override { return FadeCurve; }
	virtual const FText GetFadeCurvePropertyName() const override { return FText::FromName(GET_MEMBER_NAME_CHECKED(UFadeCurveFunctionExponential, FadeCurve)); }

	UPROPERTY(EditAnywhere, Category = "Fade", meta = (ClampMin = 1, ClampMax = 10.0, DisplayName = "Fade Curve"))
	float FadeCurve = 3.f;

protected:
	virtual const float GetFadeCurveMin() const override { return 1.f; }
	virtual const float GetFadeCurveMax() const override { return 10.f; }
};

UCLASS(MinimalAPI, NotBlueprintable, meta = (DisplayName = "Logarithmic"))
class UE_DEPRECATED(5.8, "UFadeFunction has been deprecated. Use UTransformationFadeFunction.") UFadeCurveFunctionLogarithmic : public UFadeCurveFunction
{
	GENERATED_BODY()
public:

	UE_API virtual const double GetFadeInCurveValue(const double FadeFraction) const override;
	UE_API virtual const double GetFadeOutCurveValue(const double FadeFraction) const override;

	virtual void SetFadeCurve(const float InFadeCurve) override { FadeCurve = FMath::Clamp(InFadeCurve, GetFadeCurveMin(), GetFadeCurveMax()); }
	virtual const float GetFadeCurve() const override { return FadeCurve; }
	virtual const FText GetFadeCurvePropertyName() const override { return FText::FromName(GET_MEMBER_NAME_CHECKED(UFadeCurveFunctionLogarithmic, FadeCurve)); }

	UPROPERTY(EditAnywhere, Category = "Fade", meta = (ClampMin = 0.0, ClampMax = 1, DisplayName = "Fade Curve"))
	float FadeCurve = 0.25f;
};

UCLASS(MinimalAPI, NotBlueprintable, meta = (DisplayName = "Sigmoid"))
class UE_DEPRECATED(5.8, "UFadeFunction has been deprecated. Use UTransformationFadeFunction.") UFadeCurveFunctionSigmoid : public UFadeCurveFunction
{
	GENERATED_BODY()
public:

	UE_API virtual const double GetFadeInCurveValue(const double FadeFraction) const override;
	UE_API virtual const double GetFadeOutCurveValue(const double FadeFraction) const override;

	virtual void SetFadeCurve(const float InFadeCurve) override { SFadeCurve = FMath::Clamp(InFadeCurve, GetFadeCurveMin(), GetFadeCurveMax()); }
	virtual const float GetFadeCurve() const override { return SFadeCurve; }
	virtual const FText GetFadeCurvePropertyName() const override { return FText::FromName(GET_MEMBER_NAME_CHECKED(UFadeCurveFunctionSigmoid, SFadeCurve)); }

	UPROPERTY(EditAnywhere, Category = "Fade", meta = (ClampMin = 0.0, ClampMax = 1, DisplayName = "S-Fade Curve"))
	float SFadeCurve = 0.6f;
};

USTRUCT()
struct UE_DEPRECATED(5.8, "UFadeFunction has been deprecated. Use UTransformationFadeFunction.") FFadeFunctionData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = "Fade", Instanced)
	TObjectPtr<UFadeFunction> FadeIn;

	UPROPERTY(EditAnywhere, Category = "Fade", Instanced)
	TObjectPtr<UFadeFunction> FadeOut;

	void MigrateToTransformationFadeFunction(const TObjectPtr<UWaveformTransformationFade> InFadeTransformation, FTransformationFadeFunctionData& NewFadeRegion);
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

class FWaveTransformationTrimFade : public Audio::IWaveTransformation
{
public:
	UE_API explicit FWaveTransformationTrimFade(TObjectPtr<UWaveformTransformationFade> InFadeTransformation, double InStartTime, double InEndTime);
	UE_API virtual void ProcessAudio(Audio::FWaveformTransformationWaveInfo& InOutWaveInfo) const override;

	virtual constexpr Audio::ETransformationPriority FileChangeLengthPriority() const override { return Audio::ETransformationPriority::High; }

private:

	TObjectPtr<UWaveformTransformationFade> FadeTransformation;

	double StartTime = 0.0;
	double EndTime = 0.0;
};

UCLASS(MinimalAPI)
class UWaveformTransformationTrimFade final : public UWaveformTransformationBase
{
	GENERATED_BODY()
public:

	UWaveformTransformationTrimFade(const FObjectInitializer& ObjectInitializer);

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	UE_API virtual FString GetTransformationHash() const override;
#endif // WITH_EDITOR

	UE_API virtual void PostLoad() override;

	UPROPERTY(EditAnywhere, Category = "Trim", meta=(ClampMin = 0.0))
	double StartTime = 0.0;

	UPROPERTY(EditAnywhere, Category = "Trim")
	double EndTime = -1.0;

	UPROPERTY(EditAnywhere, Instanced, Category = "Fade", meta = (DisplayName = "Fade Functions"))
	TObjectPtr<UWaveformTransformationFade> FadeTransformation = nullptr;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "UFadeFunction has been deprecated. Use UTransformationFadeFunction.")
	UPROPERTY(EditAnywhere, Category = "Fade", meta = (DisplayName = "Fade Functions", EditCondition = "false", EditConditionHides))
	FFadeFunctionData FadeFunctions;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.7, "Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.")
	/*
	 * Relative to StartTime
	 * Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.
	 */ 
	UPROPERTY(EditAnywhere, Category = "Fade", meta = (ClampMin = 0.0, DisplayName = "Fade-In Duration", EditCondition = "StartFadeTime != 0", EditConditionHides))
	float StartFadeTime = 0.f;
	
	UE_DEPRECATED(5.7, "Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.")
	/*
	 * Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.
	 */
	UPROPERTY(EditAnywhere, Category = "Fade", meta=(ClampMin = -0.1, ClampMax = 10.0, DisplayName = "Fade-In Curve", EditCondition = "StartFadeTime != 0", EditConditionHides))
	float StartFadeCurve = 1.f;

	UE_DEPRECATED(5.7, "Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.")
	/*
	 * Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.
	 */
	UPROPERTY(EditAnywhere, Category = "Fade", meta = (ClampMin = 0, ClampMax = 1.0, DisplayName = "S-Curve Sharpness", EditCondition = "StartFadeTime != 0 && StartFadeCurve < 0", EditConditionHides))
	float StartSCurveSharpness = 0.75f;

	UE_DEPRECATED(5.7, "Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.")
	/*
	 * Relative to EndTime. Fade out starts at EndTime - EndFadeTime and ends at EndTime
	 * Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.
	 */
	UPROPERTY(EditAnywhere, Category = "Fade", meta = (ClampMin = 0.0, DisplayName = "Fade-Out Duration", EditCondition = "EndFadeTime != 0", EditConditionHides))
	float EndFadeTime = 0.f;

	UE_DEPRECATED(5.7, "Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.")
	/*
	 * Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.
	 */
	UPROPERTY(EditAnywhere, Category = "Fade", meta=(ClampMin = -0.1, ClampMax = 10.0, DisplayName = "Fade-Out Curve", EditCondition = "EndFadeTime != 0", EditConditionHides))
	float EndFadeCurve = 1.f;

	UE_DEPRECATED(5.7, "Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.")
	/*
	 * Individual fade properties have been deprecated. Use UTransformationFadeFunction instead.
	 */
	UPROPERTY(EditAnywhere, Category = "Fade", meta = (ClampMin = 0, ClampMax = 1.0, DisplayName = "S-Curve Sharpness", EditCondition = "EndFadeTime != 0 && EndFadeCurve < 0", EditConditionHides))
	float EndSCurveSharpness = 0.75;

	UE_API virtual Audio::FTransformationPtr CreateTransformation() const override;

	UE_API void UpdateConfiguration(FWaveTransformUObjectConfiguration& InOutConfiguration) override;
	virtual constexpr Audio::ETransformationPriority GetTransformationPriority() const { return Audio::ETransformationPriority::High; }	

	UE_API const FTransformationFadeFunctionData& GetFadeRegion() const;
	UE_API FTransformationFadeFunctionData& GetMutableFadeRegion() const;
	
private:	

	UE_API void UpdateDurationProperties(const float InAvailableDuration);
	float AvailableWaveformDuration = -1.f;
};

#undef UE_API
