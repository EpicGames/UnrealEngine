// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class WaveformTransformations : ModuleRules
	{
		public WaveformTransformations(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"AudioExtensions",
					"AudioSynesthesiaCore",
					"Core", 
					"CoreUObject",
					"Engine",
					"InputCore",
					"PropertyEditor",
					"SignalProcessing",
					"UnrealEd"
				}
			);
		}
	}
}