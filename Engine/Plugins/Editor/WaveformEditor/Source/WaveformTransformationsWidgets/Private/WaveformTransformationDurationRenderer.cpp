// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationDurationRenderer.h"

FWaveformTransformationDurationRenderer::FWaveformTransformationDurationRenderer(const uint32 InOriginalWaveformNumFrames)
	: OriginalWaveformNumFrames(InOriginalWaveformNumFrames)
{
}

int32 FWaveformTransformationDurationRenderer::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// If there is a null transformation in Transformations list, the TransformationWaveInfo has not been set up
	if (TransformationWaveInfo.NumChannels <= 0)
	{
		return LayerId;
	}

	float OriginalWaveformNumFramesDen = static_cast<float>(OriginalWaveformNumFrames * TransformationWaveInfo.NumChannels);
	check(OriginalWaveformNumFramesDen > 0.f);

	const float StartTimeRatio = static_cast<float>(TransformationWaveInfo.StartFrameOffset) / OriginalWaveformNumFramesDen;
	const int64 EndSample = TransformationWaveInfo.StartFrameOffset + static_cast<int64>(TransformationWaveInfo.TotalNumSamples);
	const float EndTimeRatio = static_cast<float>(EndSample) / OriginalWaveformNumFramesDen;

	const bool bRenderLeftBox = StartTimeRatio >= 0.f;
	const bool bRenderRightBox = EndTimeRatio <= 1.f;

	if (bRenderLeftBox)
	{
		const float RightMarginX = StartTimeRatio * AllottedGeometry.Size.X;

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(FVector2D(RightMarginX, AllottedGeometry.Size.Y), FSlateLayoutTransform()),
			FAppStyle::GetBrush(WaveformTransformationDurationHiglightParams::BackgroundBrushName),
			ESlateDrawEffect::None,
			WaveformTransformationDurationHiglightParams::BoxColor);

	}

	if (bRenderRightBox)
	{
		const float LeftMarginX = EndTimeRatio * AllottedGeometry.Size.X;

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(FVector2D(AllottedGeometry.Size.X, AllottedGeometry.Size.Y), FSlateLayoutTransform(FVector2D(LeftMarginX, 0.f))),
			FAppStyle::GetBrush(WaveformTransformationDurationHiglightParams::BackgroundBrushName),
			ESlateDrawEffect::None,
			WaveformTransformationDurationHiglightParams::BoxColor);

	}

	return LayerId;
}

void FWaveformTransformationDurationRenderer::SetOriginalWaveformFrames(const uint32 NumFrames)
{
	OriginalWaveformNumFrames = NumFrames;
}
