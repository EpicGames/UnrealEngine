// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationFadeRenderer.h"

#include "AudioWidgetsStyle.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "Layout/Geometry.h"
#include "PropertyHandle.h"
#include "Rendering/DrawElements.h"
#include "Sound/SoundWave.h"
#include "WaveformTransformationLog.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"
#include "Blueprint/WidgetLayoutLibrary.h"

namespace WaveformFadeRendererColors
{
	const FLinearColor FadeOutColor = FLinearColor(1.f, 0.1f, 0.f, 1.f);
	const FLinearColor FadeInColor = FLinearColor::Green;
	const FLinearColor SelectionColor = FLinearColor::Yellow;
	const FLinearColor LineColor = FLinearColor::Gray;
}

FWaveformTransformationFadeRenderer::~FWaveformTransformationFadeRenderer()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().OnFocusChanging().Remove(PopupHandle);
		FSlateApplication::Get().OnApplicationActivationStateChanged().Remove(ApplicationActivationStateHandle);
	}

	OnFadeModeChanged.Unbind();
}

int32 FWaveformTransformationFadeRenderer::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = DrawOffsetHandles(AllottedGeometry, OutDrawElements, LayerId);
	LayerId = DrawFadeCurves(AllottedGeometry, OutDrawElements, LayerId);

	return LayerId;
}

int32 FWaveformTransformationFadeRenderer::DrawOffsetHandles(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	const float HandleWidth = InteractionHandleSize;

	for (const FFadeRegion& FadeRegion : FadeRegions)
	{
		FPaintGeometry HandleGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(HandleWidth, HandleWidth), FSlateLayoutTransform(FVector2f(FadeRegion.FadeInStartX, AllottedGeometry.Size.Y - HandleWidth)));

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			HandleGeometry,
			&RoundedBoxBrush,
			ESlateDrawEffect::None,
			WaveformFadeRendererColors::FadeInColor
		);

		HandleGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(HandleWidth, HandleWidth), FSlateLayoutTransform(FVector2f(FadeRegion.FadeOutEndX - HandleWidth, AllottedGeometry.Size.Y - HandleWidth)));

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			HandleGeometry,
			&RoundedBoxBrush,
			ESlateDrawEffect::None,
			WaveformFadeRendererColors::FadeOutColor
		);
	}

	return LayerId;
}

int32 FWaveformTransformationFadeRenderer::DrawFadeCurves(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	const float HandleWidth = InteractionHandleSize;
	const float HandleHeight = InteractionHandleSize;
	const float HandleOffset = 0.0f;

	const FSlateBrush* FadeInHandleBrush = FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeStart").GetIcon();
	const FSlateBrush* FadeOutHandleBrush = FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeEnd").GetIcon();

	FPaintGeometry HandleGeometry;

	for (const FFadeRegion& FadeRegion : FadeRegions)
	{
		const bool IsFadeInSelected = FadeRegion.FadeInEndInteractionXRange.Contains(MousePosition.X) && FadeRegion.FadeInEndInteractionYRange.Contains(MousePosition.Y);
		const bool IsFadeOutSelected = FadeRegion.FadeOutStartInteractionXRange.Contains(MousePosition.X) && FadeRegion.FadeOutStartInteractionYRange.Contains(MousePosition.Y);

		if (FadeRegion.FadeInCurvePoints.Num() > 0)
		{
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				++LayerId,
				AllottedGeometry.ToPaintGeometry(),
				FadeRegion.FadeInCurvePoints,
				ESlateDrawEffect::None,
				WaveformFadeRendererColors::LineColor
			);

			const float HandleStart = FadeRegion.FadeInCurvePoints.Last(0).X;
			HandleGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(HandleWidth, HandleHeight), FSlateLayoutTransform(FVector2f(HandleStart - HandleOffset, 0)));
		}
		else
		{
			HandleGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(HandleWidth, HandleHeight), FSlateLayoutTransform(FVector2f(FadeRegion.FadeInStartX- HandleOffset, 0)));
		}

		check(FadeInHandleBrush);
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			HandleGeometry,
			FadeInHandleBrush,
			ESlateDrawEffect::None,
			IsFadeInSelected ? WaveformFadeRendererColors::SelectionColor : WaveformFadeRendererColors::FadeInColor
		);

		if (FadeRegion.FadeOutCurvePoints.Num() > 0)
		{
			FSlateDrawElement::MakeLines(
				OutDrawElements,
				++LayerId,
				AllottedGeometry.ToPaintGeometry(),
				FadeRegion.FadeOutCurvePoints,
				ESlateDrawEffect::None,
				WaveformFadeRendererColors::LineColor
			);

			const float HandleStart = FadeRegion.FadeOutCurvePoints[0].X - HandleWidth;
			HandleGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(HandleWidth, HandleHeight), FSlateLayoutTransform(FVector2f(HandleStart + HandleOffset, 0)));
		}
		else
		{
			HandleGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(HandleWidth, HandleHeight), FSlateLayoutTransform(FVector2f(FadeRegion.FadeOutEndX - HandleWidth + HandleOffset, 0)));
		}

		check(FadeOutHandleBrush);
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			HandleGeometry,
			FadeOutHandleBrush,
			ESlateDrawEffect::None,
			IsFadeOutSelected ? WaveformFadeRendererColors::SelectionColor : WaveformFadeRendererColors::FadeOutColor
		);
	}

	return LayerId;
}

void FWaveformTransformationFadeRenderer::GenerateFadeCurves(const FGeometry& AllottedGeometry)
{
	check(StrongFade);
	check(FadeRegions.Num() == StrongFade->GetFadeRegionsNum());
	check(TransformationWaveInfo.NumChannels > 0);

	const int64 NumFrames = (TransformationWaveInfo.NumSamplesAvailable / TransformationWaveInfo.NumChannels);

	for (int32 Index = 0; Index < FadeRegions.Num(); Index++)
	{
		check(FadeRegions[Index].FadeInStartX >= 0);
		check(FadeRegions[Index].FadeInEndX >= 0);
		check(StrongFade->GetFadeRegions()[Index].FadeIn);
		check(StrongFade->GetFadeRegions()[Index].FadeOut);

		if (StrongFade->GetFadeRegions()[Index].FadeIn->Duration > 0.f && PixelsPerFrame > 0.0)
		{
			const float FadeInTime = StrongFade->GetFadeRegions()[Index].FadeIn->Duration;
			const float FadeInFrames = FadeInTime * TransformationWaveInfo.SampleRate;
			const float FadeInPixelLength = FadeInFrames * PixelsPerFrame;
			check(FadeInPixelLength > 0.f);

			const uint32 DisplayedFadeInPixelLength = 
				(FadeRegions[Index].FadeInStartX < FadeRegions[Index].FadeInEndX) ? (FadeRegions[Index].FadeInEndX - FadeRegions[Index].FadeInStartX) : 0;

			if (DisplayedFadeInPixelLength > (NumFrames * PixelsPerFrame))
			{
				// Mid property change, the DisplayedFadeInPixelLength can become invalid here. It will be updated next tick
				return;
			}

			FadeRegions[Index].FadeInCurvePoints.SetNumUninitialized(DisplayedFadeInPixelLength);

			if (DisplayedFadeInPixelLength > 0)
			{
				// Ensure the first pixel of the line is placed vertically at the bottom to correctly show where the fade starts
				FadeRegions[Index].FadeInCurvePoints[0] = FVector2D(FadeRegions[Index].FadeInStartX, AllottedGeometry.Size.Y);

				for (uint32 Pixel = 1; Pixel < DisplayedFadeInPixelLength; ++Pixel)
				{
					const double FadeFraction = FMath::Clamp(static_cast<float>(Pixel) / FadeInPixelLength, 0.f, 1.f);
					const double CurveFunction = StrongFade->GetFadeRegions()[Index].FadeIn->GetFadeInCurveValue(FadeFraction);

					// Prevents line flicker when dragging the fade handle
					const bool bNeedsFlickerPrevention = (Pixel == DisplayedFadeInPixelLength - 1) && DisplayedFadeInPixelLength != 1;

					// Prevent fade line from flickering if the math doesn't work out to the last Y value being 0 as the position is changing
					// This is due to pixel rounding errors. The slight distortion caused by this correction isn't noticeable
					const double CurveValue = bNeedsFlickerPrevention ? 0.f : 1.f - CurveFunction;

					const uint32 XCoordinate = Pixel + FadeRegions[Index].FadeInStartX;
					FadeRegions[Index].FadeInCurvePoints[Pixel] = FVector2D(XCoordinate, CurveValue * AllottedGeometry.Size.Y);
				}

				check(FadeRegions[Index].FadeInCurvePoints.Num() > 0);
				check(FadeRegions[Index].FadeInCurvePoints[0].Y == AllottedGeometry.Size.Y);
			}
		}
		else
		{
			FadeRegions[Index].FadeInCurvePoints.SetNumUninitialized(0);
		}

		if (StrongFade->GetFadeRegions()[Index].FadeOut->Duration > 0.f && PixelsPerFrame > 0.0)
		{
			const float FadeOutTime = StrongFade->GetFadeRegions()[Index].FadeOut->Duration;
			const float FadeOutFrames = FadeOutTime * TransformationWaveInfo.SampleRate;
			const float FadeOutPixelLength = FadeOutFrames * PixelsPerFrame;
			check(FadeOutPixelLength > 0.f);

			const uint32 DisplayedFadeOutPixelLength = 
				(FadeRegions[Index].FadeOutStartX < FadeRegions[Index].FadeOutEndX) ? (FadeRegions[Index].FadeOutEndX - FadeRegions[Index].FadeOutStartX) : 0;

			if (DisplayedFadeOutPixelLength > (NumFrames * PixelsPerFrame))
			{
				// Mid property change, the DisplayedFadeInPixelLength can become invalid here. It will be updated next tick
				return;
			}

			FadeRegions[Index].FadeOutCurvePoints.SetNumUninitialized(DisplayedFadeOutPixelLength);

			if (DisplayedFadeOutPixelLength > 0)
			{
				// Ensure the first pixel of the line is placed vertically at the top to correctly show where the fade starts
				FadeRegions[Index].FadeOutCurvePoints[0] = FVector2D(FadeRegions[Index].FadeOutStartX, 0);

				for (uint32 Pixel = 1; Pixel < DisplayedFadeOutPixelLength; ++Pixel)
				{
					const double FadeFraction = FMath::Clamp(static_cast<float>(Pixel) / FadeOutPixelLength, 0.f, 1.f);
					const double CurveFunction = StrongFade->GetFadeRegions()[Index].FadeOut->GetFadeOutCurveValue(FadeFraction);

					// Prevents line flicker when dragging the fade handle and prevents problems with the logarithmic fade when using a 0 exponent value.
					const bool bNeedsFlickerPrevention = (Pixel == DisplayedFadeOutPixelLength - 1) && DisplayedFadeOutPixelLength != 1;

					// Prevent fade line from flickering if the math doesn't work out to the last Y value being equal to AllottedGeometry.Size.Y as the 
					// position is changing. This is due to pixel rounding errors. The slight distortion caused by this correction isn't noticeable
					const double CurveValue = bNeedsFlickerPrevention ? 1.f : 1.f - CurveFunction;

					const uint32 XCoordinate = Pixel + FadeRegions[Index].FadeOutStartX;
					FadeRegions[Index].FadeOutCurvePoints[Pixel] = FVector2D(XCoordinate, CurveValue * AllottedGeometry.Size.Y);
				}

				check(FadeRegions[Index].FadeOutCurvePoints.Num() > 0);
				check(FadeRegions[Index].FadeOutCurvePoints[0].Y == 0.f);
			}
		}
		else
		{
			FadeRegions[Index].FadeOutCurvePoints.SetNumUninitialized(0);
		}
	}
}

FCursorReply FWaveformTransformationFadeRenderer::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	const FVector2D LocalCursorPosition = GetLocalCursorPosition(CursorEvent, MyGeometry);

	switch (FadeInteractionTypeAndIndex.Key)
	{
	case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeInStart:
	case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeInEnd:
	case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeOutStart:
	case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeOutEnd:
		return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
	default:
		break;
	}

	if (IsCursorInFadeInteractionRange(LocalCursorPosition, MyGeometry))
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHand);
	}

	return FCursorReply::Unhandled();
}

void FWaveformTransformationFadeRenderer::SetWaveformTransformation(TObjectPtr<UWaveformTransformationBase> InTransformation)
{
	FWaveformTransformationRendererBase::SetWaveformTransformation(InTransformation);
	
	StrongFade = TStrongObjectPtr<UWaveformTransformationFade>(CastChecked<UWaveformTransformationFade>(InTransformation.Get()));

	FadeRegions.Reset();
	FadeRegions.SetNum(StrongFade->GetFadeRegionsNum());
}

FReply FWaveformTransformationFadeRenderer::OnMouseWheel(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	check(StrongFade);
	check(FadeRegions.Num() == StrongFade->GetFadeRegionsNum());

	// Prevent stomping on other mouse transactions
	if (FadeInteractionTypeAndIndex.Key != EFadeInteractionType::None)
	{
		return FReply::Handled();
	}

	const FVector2D LocalCursorPosition = GetLocalCursorPosition(MouseEvent, MyGeometry);

	for (int32 Index = 0; Index < FadeRegions.Num(); Index++)
	{
		check(StrongFade->GetFadeRegions()[Index].FadeIn);
		check(StrongFade->GetFadeRegions()[Index].FadeOut);

		if (FadeRegions[Index].FadeInEndInteractionXRange.Contains(LocalCursorPosition.X) 
			&& FadeRegions[Index].FadeInEndInteractionYRange.Contains(LocalCursorPosition.Y) 
			&& StrongFade->GetFadeRegions()[Index].FadeIn.IsA<UTransformationFadeCurveFunction>())
		{
			const float FadeCurveDelta = MouseEvent.GetWheelDelta() * MouseWheelStep;
			FText OutPropertyName = FText::FromName(NAME_None);

			TObjectPtr<UTransformationFadeCurveFunction> FadeIn =
				CastChecked<UTransformationFadeCurveFunction>(StrongFade->GetFadeRegions()[Index].FadeIn);
			OutPropertyName = FadeIn->GetFadeCurvePropertyName();

			if (!GIsTransacting)
			{
				TransactionResult =
					BeginTransaction(TEXT("PropertyEditor"), FText::Format(NSLOCTEXT("PropertyEditor", "SetNumericPropertyTransaction", "Edit {0}"), OutPropertyName), StrongFade.Get());

				if (TransactionResult == INDEX_NONE)
				{
					UE_LOGF(LogWaveformTransformation, Warning, "Begin FadeTransformation OnMouseWheel Transaction Failed");
				}
			}

			StrongFade->Modify();
			StrongFade->GetFadeRegions()[Index].FadeIn->Modify();

			const float FadeCurveValue = FadeIn->GetFadeCurve() + FadeCurveDelta;
			FadeIn->SetFadeCurve(FadeCurveValue);

			if (TransactionResult != INDEX_NONE)
			{
				EndTransaction();
				TransactionResult = INDEX_NONE;
			}

			StrongFade->OnTransformationChanged.ExecuteIfBound(true);

			return FReply::Handled();
		}

		if (FadeRegions[Index].FadeOutStartInteractionXRange.Contains(LocalCursorPosition.X) 
			&& FadeRegions[Index].FadeOutStartInteractionYRange.Contains(LocalCursorPosition.Y) 
			&& StrongFade->GetFadeRegions()[Index].FadeOut.IsA<UTransformationFadeCurveFunction>())
		{
			const float FadeCurveDelta = MouseEvent.GetWheelDelta() * MouseWheelStep;
			FText OutPropertyName = FText::FromName(NAME_None);

			TObjectPtr<UTransformationFadeCurveFunction> FadeOut =
				CastChecked<UTransformationFadeCurveFunction>(StrongFade->GetFadeRegions()[Index].FadeOut);
			OutPropertyName = FadeOut->GetFadeCurvePropertyName();
			
			if (!GIsTransacting)
			{
				TransactionResult =
					BeginTransaction(TEXT("PropertyEditor"), FText::Format(NSLOCTEXT("PropertyEditor", "SetNumericPropertyTransaction", "Edit {0}"), OutPropertyName), StrongFade.Get());

				if (TransactionResult == INDEX_NONE)
				{
					UE_LOGF(LogWaveformTransformation, Warning, "Begin FadeTransformation OnMouseWheel Transaction Failed");
				}
			}

			StrongFade->Modify();
			StrongFade->GetFadeRegions()[Index].FadeOut->Modify();

			const float FadeCurveValue = FadeOut->GetFadeCurve() + FadeCurveDelta;
			FadeOut->SetFadeCurve(FadeCurveValue);

			if (TransactionResult != INDEX_NONE)
			{
				EndTransaction();
				TransactionResult = INDEX_NONE;
			}

			StrongFade->OnTransformationChanged.ExecuteIfBound(true);

			return FReply::Handled();
		}
	}

	return FReply::Unhandled();
}

FVector2D FWaveformTransformationFadeRenderer::GetLocalCursorPosition(const FPointerEvent& MouseEvent, const FGeometry& EventGeometry) const
{
	const FVector2D ScreenSpacePosition = MouseEvent.GetScreenSpacePosition();
	return  EventGeometry.AbsoluteToLocal(ScreenSpacePosition);
}

double FWaveformTransformationFadeRenderer::ConvertXRatioToTime(const float InRatio) const
{
	check(TransformationWaveInfo.NumChannels > 0);
	check(TransformationWaveInfo.SampleRate > 0.f);

	const float NumFrames = TransformationWaveInfo.TotalNumSamples / TransformationWaveInfo.NumChannels;
	const float FrameSelected = NumFrames * InRatio;
	return FrameSelected / TransformationWaveInfo.SampleRate;
}

void FWaveformTransformationFadeRenderer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{ 
	check(TransformationWaveInfo.NumChannels > 0);
	check(StrongFade);

	const float NumFrames = TransformationWaveInfo.TotalNumSamples / TransformationWaveInfo.NumChannels;

	check(NumFrames > 0);
	PixelsPerFrame = FMath::Max(AllottedGeometry.GetLocalSize().X / NumFrames, 0.0);

	const int32 NumFadeRegions = StrongFade->GetFadeRegionsNum();
	if (NumFadeRegions != FadeRegions.Num())
	{
		FadeRegions.SetNum(NumFadeRegions);
	}

	check(NumFadeRegions == FadeRegions.Num());

	for (int32 Index = 0; Index < NumFadeRegions; Index++)
	{
		check(StrongFade->GetFadeRegions()[Index].FadeIn);
		check(StrongFade->GetFadeRegions()[Index].FadeOut);

		FadeRegions[Index].FadeInStartX = static_cast<uint32>(FMath::Clamp(StrongFade->GetFadeRegions()[Index].FadeIn->FrameOffset * PixelsPerFrame, 0, UINT32_MAX));
		FadeRegions[Index].FadeOutEndX = static_cast<uint32>(FMath::Clamp((NumFrames * PixelsPerFrame) - (StrongFade->GetFadeRegions()[Index].FadeOut->FrameOffset * PixelsPerFrame), 0, UINT32_MAX));
		FadeRegions[Index].FadeInEndX = static_cast<uint32>(FMath::Clamp(StrongFade->GetFadeRegions()[Index].FadeIn->Duration * TransformationWaveInfo.SampleRate * PixelsPerFrame + FadeRegions[Index].FadeInStartX, 0, UINT32_MAX));
		FadeRegions[Index].FadeOutStartX = static_cast<uint32>(FMath::Clamp(FadeRegions[Index].FadeOutEndX - (StrongFade->GetFadeRegions()[Index].FadeOut->Duration * TransformationWaveInfo.SampleRate * PixelsPerFrame), 0, UINT32_MAX));
	}

	FVector2D MouseAbsolutePosition = UWidgetLayoutLibrary::GetMousePositionOnPlatform();
	MousePosition = AllottedGeometry.AbsoluteToLocal(MouseAbsolutePosition);

	GenerateFadeCurves(AllottedGeometry);
	UpdateInteractionRange(AllottedGeometry);
}

void FWaveformTransformationFadeRenderer::UpdateInteractionRange(const FGeometry& AllottedGeometry)
{
	for (FFadeRegion& FadeRegion : FadeRegions)
	{
		FadeRegion.FadeInStartInteractionXRange.SetLowerBoundValue(FadeRegion.FadeInStartX - InteractionHandleSize);
		FadeRegion.FadeInStartInteractionXRange.SetUpperBoundValue(FadeRegion.FadeInStartX + InteractionHandleSize);
		FadeRegion.FadeInEndInteractionXRange.SetLowerBoundValue(FadeRegion.FadeInEndX - InteractionHandleSize);
		FadeRegion.FadeInEndInteractionXRange.SetUpperBoundValue(FadeRegion.FadeInEndX + InteractionHandleSize);
		FadeRegion.FadeInStartInteractionYRange.SetLowerBoundValue(AllottedGeometry.GetLocalSize().Y - InteractionHandleSize);
		FadeRegion.FadeInStartInteractionYRange.SetUpperBoundValue(AllottedGeometry.GetLocalSize().Y);
		FadeRegion.FadeInEndInteractionYRange.SetLowerBoundValue(0);
		FadeRegion.FadeInEndInteractionYRange.SetUpperBoundValue(InteractionHandleSize);

		FadeRegion.FadeOutStartInteractionXRange.SetLowerBoundValue(FadeRegion.FadeOutStartX - InteractionHandleSize);
		FadeRegion.FadeOutStartInteractionXRange.SetUpperBoundValue(FadeRegion.FadeOutStartX + InteractionHandleSize);
		FadeRegion.FadeOutEndInteractionXRange.SetLowerBoundValue(FadeRegion.FadeOutEndX - InteractionHandleSize);
		FadeRegion.FadeOutEndInteractionXRange.SetUpperBoundValue(FadeRegion.FadeOutEndX + InteractionHandleSize);
		FadeRegion.FadeOutStartInteractionYRange.SetLowerBoundValue(0);
		FadeRegion.FadeOutStartInteractionYRange.SetUpperBoundValue(InteractionHandleSize);
		FadeRegion.FadeOutEndInteractionYRange.SetLowerBoundValue(AllottedGeometry.GetLocalSize().Y - InteractionHandleSize);
		FadeRegion.FadeOutEndInteractionYRange.SetUpperBoundValue(AllottedGeometry.GetLocalSize().Y);
	}
}

bool FWaveformTransformationFadeRenderer::IsCursorInFadeInteractionRange(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry) const
{
	for (const FFadeRegion& FadeRegion : FadeRegions)
	{
		if ((FadeRegion.FadeInStartInteractionXRange.Contains(InLocalCursorPosition.X) && FadeRegion.FadeInStartInteractionYRange.Contains(InLocalCursorPosition.Y))
			|| (FadeRegion.FadeInEndInteractionXRange.Contains(InLocalCursorPosition.X) && FadeRegion.FadeInEndInteractionYRange.Contains(InLocalCursorPosition.Y))
			|| (FadeRegion.FadeOutStartInteractionXRange.Contains(InLocalCursorPosition.X) && FadeRegion.FadeOutStartInteractionYRange.Contains(InLocalCursorPosition.Y))
			|| (FadeRegion.FadeOutEndInteractionXRange.Contains(InLocalCursorPosition.X) && FadeRegion.FadeOutEndInteractionYRange.Contains(InLocalCursorPosition.Y)))
		{
			return true;
		}
	}

	return false;
}

FReply FWaveformTransformationFadeRenderer::OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	check(StrongFade);
	const FVector2D LocalCursorPosition = GetLocalCursorPosition(MouseEvent, MyGeometry);

	const FKey MouseButton = MouseEvent.GetEffectingButton();

	FadeInteractionTypeAndIndex = GetInteractionTypeAndIndexFromCursorPosition(LocalCursorPosition, MouseButton, MyGeometry);

	if (FadeInteractionTypeAndIndex.Key != EFadeInteractionType::None)
	{
		if (!GIsTransacting)
		{
			TransactionResult =
				BeginTransaction(TEXT("PropertyEditor"), FText::Format(NSLOCTEXT("PropertyEditor", "SetNumericPropertyTransaction", "Edit {0}"), GetPropertyEditedByCurrentInteraction()), StrongFade.Get());

			if (TransactionResult == INDEX_NONE)
			{
				UE_LOGF(LogWaveformTransformation, Warning, "Begin FadeTransformation OnMouseButtonDown Transaction Failed");
			}
		}

		if (StrongFade->GetFadeRegions().IsValidIndex(FadeInteractionTypeAndIndex.Value))
		{
			check(StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeIn);
			check(StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeOut);

			if ((FadeInteractionTypeAndIndex.Key == EFadeInteractionType::ScrubbingFadeInEnd 
				|| FadeInteractionTypeAndIndex.Key == EFadeInteractionType::ScrubbingFadeInStart))
			{
				StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeIn->Modify();
			}
			else if ((FadeInteractionTypeAndIndex.Key == EFadeInteractionType::ScrubbingFadeOutEnd 
				|| FadeInteractionTypeAndIndex.Key == EFadeInteractionType::ScrubbingFadeOutStart))
			{
				StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeOut->Modify();
			}
		}

		StrongFade->Modify();

		return FReply::Handled().CaptureMouse(OwnerWidget.AsShared()).PreventThrottling();
	}

	return FReply::Unhandled();
}


FReply FWaveformTransformationFadeRenderer::OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	check(StrongFade);
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && FadeInteractionTypeAndIndex.Key != EFadeInteractionType::None)
	{
		SetPropertyValueDependingOnInteractionType(MouseEvent, MyGeometry, EPropertyChangeType::Interactive);

		return FReply::Handled().CaptureMouse(OwnerWidget.AsShared());
	}

	return FReply::Unhandled();
}

FReply FWaveformTransformationFadeRenderer::OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	check(StrongFade);
	if (FadeInteractionTypeAndIndex.Key != EFadeInteractionType::None)
	{
		SetPropertyValueDependingOnInteractionType(MouseEvent, MyGeometry);

		if (TransactionResult != INDEX_NONE)
		{
			EndTransaction();
			TransactionResult = INDEX_NONE;
		}

		FadeInteractionTypeAndIndex.Key = EFadeInteractionType::None;
		FadeInteractionTypeAndIndex.Value = 0;

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

void FWaveformTransformationFadeRenderer::SetPropertyValueDependingOnInteractionType(const FPointerEvent& MouseEvent, const FGeometry& WidgetGeometry, const EPropertyChangeType::Type ChangeType)
{
	check(StrongFade);
	check(PixelsPerFrame > 0.0);
	check(StrongFade->GetSampleRate() > 0);
	check(WidgetGeometry.GetLocalSize().X > 0.f);
	const FVector2D LocalCursorPosition = GetLocalCursorPosition(MouseEvent, WidgetGeometry);
	const float LocalCursorXRatio = FMath::Clamp(LocalCursorPosition.X / WidgetGeometry.GetLocalSize().X, 0.f, 1.f);
	const double SelectedTime = ConvertXRatioToTime(LocalCursorXRatio);
	const float NumFrames = TransformationWaveInfo.TotalNumSamples / TransformationWaveInfo.NumChannels;
	const float MaxDuration = NumFrames / StrongFade->GetSampleRate();

	if (StrongFade->GetFadeRegions().IsValidIndex(FadeInteractionTypeAndIndex.Value))
	{
		check(StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeIn);
		check(StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeOut);

		FProperty* Property = nullptr;

		switch (FadeInteractionTypeAndIndex.Key)
		{
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::None:
			break;
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeInStart:
		{
			StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeIn->FrameOffset = FMath::FloorToInt64(LocalCursorPosition.X / PixelsPerFrame);
			Property = UWaveformTransformationFade::StaticClass()->FindPropertyByName(UWaveformTransformationFade::GetFadeRegionsPropertyName());

			break;
		}
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeInEnd:
		{
			float StartFadeDurationValue = 0;
			StartFadeDurationValue = FMath::Clamp(SelectedTime - (FadeRegions[FadeInteractionTypeAndIndex.Value].FadeInStartX / PixelsPerFrame) / StrongFade->GetSampleRate(), 0.f, MaxDuration);
			StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeIn->Duration = StartFadeDurationValue;
			Property = UWaveformTransformationFade::StaticClass()->FindPropertyByName(UWaveformTransformationFade::GetFadeRegionsPropertyName());

			break;
		}
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeOutStart:
		{
			float EndFadeDurationValue = 0;
			const float diff = ((FadeRegions[FadeInteractionTypeAndIndex.Value].FadeOutEndX / PixelsPerFrame) / StrongFade->GetSampleRate()) - SelectedTime;
			EndFadeDurationValue = FMath::Clamp(diff, 0.f, MaxDuration);
			StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeOut->Duration = EndFadeDurationValue;
			Property = UWaveformTransformationFade::StaticClass()->FindPropertyByName(UWaveformTransformationFade::GetFadeRegionsPropertyName());

			break;
		}
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeOutEnd:
		{
			StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeOut->FrameOffset = FMath::FloorToInt64(NumFrames - LocalCursorPosition.X / PixelsPerFrame);
			Property = UWaveformTransformationFade::StaticClass()->FindPropertyByName(UWaveformTransformationFade::GetFadeRegionsPropertyName());

			break;
		}
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::RightClickFadeIn:
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::RightClickFadeOut:
			ShowSelectFadeModeMenuAtCursor(WidgetGeometry, MouseEvent);
			break;
		default:
			break;
		}

		if (Property)
		{
			FPropertyChangedEvent PropertyChangedEvent(Property, ChangeType);
			StrongFade->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
}

void FWaveformTransformationFadeRenderer::ShowSelectFadeModeMenuAtCursor(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!FSlateApplication::IsInitialized())
	{
		UE_LOGF(LogWaveformTransformation, Warning, "FSlateApplication not initialized in WaveformTransformationTrimFadeRenderer.");

		return;
	}

	const FVector2D LocalWindowMaxPosition = MyGeometry.GetAbsolutePosition() + MyGeometry.GetAbsoluteSize();
	const FVector2D LocalCursorPosition = MouseEvent.GetScreenSpacePosition();
	
	if (TSharedPtr<SWindow> LockedFadeModeMenuWindow = FadeModeMenuWindow.Pin())
	{
		FSlateApplication::Get().RequestDestroyWindow(LockedFadeModeMenuWindow.ToSharedRef());
		FSlateApplication::Get().OnFocusChanging().Remove(PopupHandle); 
		FSlateApplication::Get().OnApplicationActivationStateChanged().Remove(ApplicationActivationStateHandle);
	}

	TSharedRef<SVerticalBox> MenuContent = SNew(SVerticalBox);

	TSharedRef<SWindow> MenuWindow = SNew(SWindow)
		.AutoCenter(EAutoCenter::None)
		.ScreenPosition(LocalCursorPosition)
		.SizingRule(ESizingRule::Autosized)
		.SupportsTransparency(EWindowTransparency::None)
		.IsPopupWindow(true)
		.CreateTitleBar(false)
		[
			MenuContent
		];

	FadeModeMenuWindow = MenuWindow;
	TWeakObjectPtr<UWaveformTransformationFade> InWeakFade = StrongFade.Get();

	TSharedPtr<FDelegateHandle> ApplicationActivationStateHandlePtr = MakeShared<FDelegateHandle>();
	*ApplicationActivationStateHandlePtr = FSlateApplication::Get().OnApplicationActivationStateChanged().AddLambda([InFadeModeMenuWindow = FadeModeMenuWindow, ApplicationActivationStateHandlePtr](bool isActive)
		{
			if (!isActive)
			{
				if (TSharedPtr<SWindow> LockedFadeModeMenuWindow = InFadeModeMenuWindow.Pin())
				{
					FSlateApplication::Get().RequestDestroyWindow(LockedFadeModeMenuWindow.ToSharedRef());
					FSlateApplication::Get().OnApplicationActivationStateChanged().Remove(*ApplicationActivationStateHandlePtr);
				}
			}
		});

	if (ApplicationActivationStateHandle.IsValid())
	{
		FSlateApplication::Get().OnApplicationActivationStateChanged().Remove(ApplicationActivationStateHandle);
	}

	ApplicationActivationStateHandle = *ApplicationActivationStateHandlePtr;

	// If focus is lost on the popup, destroy it to prevent popups hanging around
	TSharedPtr<FDelegateHandle> PopupHandlePtr = MakeShared<FDelegateHandle>();
	*PopupHandlePtr = FSlateApplication::Get().OnFocusChanging().AddLambda([InFadeModeMenuWindow = FadeModeMenuWindow, PopupHandlePtr,
		InApplicationActivationStateHandle = ApplicationActivationStateHandle](const FFocusEvent& FocusEvent, const FWeakWidgetPath& WeakWidgetPath
		, const TSharedPtr<SWidget>& OldWidget, const FWidgetPath& WidgetPath, const TSharedPtr<SWidget>& NewWidget)
		{
			if (InFadeModeMenuWindow != nullptr && InFadeModeMenuWindow.IsValid())
			{
				if (TSharedPtr<SWindow> LockedFadeModeMenuWindow = InFadeModeMenuWindow.Pin())
				{
					if (OldWidget && !OldWidget->IsHovered() && LockedFadeModeMenuWindow == OldWidget)
					{
						FSlateApplication::Get().RequestDestroyWindow(LockedFadeModeMenuWindow.ToSharedRef());
						FSlateApplication::Get().OnFocusChanging().Remove(*PopupHandlePtr);

						if (InApplicationActivationStateHandle.IsValid())
						{
							FSlateApplication::Get().OnApplicationActivationStateChanged().Remove(InApplicationActivationStateHandle);
						}
					}
				}
			}
		});

	if (PopupHandle.IsValid())
	{
		FSlateApplication::Get().OnFocusChanging().Remove(PopupHandle);
	}

	PopupHandle = *PopupHandlePtr;

	for (const TPair<EWaveEditorTransformationFadeMode, TSubclassOf<UTransformationFadeFunction>>& FadeOptionPair : UWaveformTransformationFade::FadeModeToFadeFunctionMap)
	{
		MenuContent->AddSlot()
			.Padding(5)
			.AutoHeight()
			[
				SNew(SButton)
					.OnClicked_Lambda([InPopupHandle = PopupHandle, InApplicationActivationStateHandle = ApplicationActivationStateHandle, InOnFadeModeChanged = OnFadeModeChanged,
						InWeakFadeModeMenuWindow = FadeModeMenuWindow, InWeakFade, InteractionTypeAndIndex = FadeInteractionTypeAndIndex,
						FadeOptionPair, PropertyName = GetPropertyEditedByCurrentInteraction()]() -> FReply
					{
						if (TStrongObjectPtr<UWaveformTransformationFade> PinnedFade = InWeakFade.Pin())
						{
							const int32 Index = InteractionTypeAndIndex.Value;

							if (Index < PinnedFade->GetFadeRegionsNum())
							{
								if (GEditor && GEditor->Trans)
								{
									GEditor->BeginTransaction(TEXT("PropertyEditor"), FText::Format(NSLOCTEXT("PropertyEditor", "SetCurveProperty", "Set Curve"), PropertyName), PinnedFade.Get());
								}

								PinnedFade->Modify();

								if (InteractionTypeAndIndex.Key == FWaveformTransformationFadeRenderer::EFadeInteractionType::RightClickFadeIn)
								{
									if (PinnedFade->GetFadeRegions()[Index].FadeIn)
									{
										const float Duration = PinnedFade->GetFadeRegions()[Index].FadeIn->Duration;
										const int64 Offset = PinnedFade->GetFadeRegions()[Index].FadeIn->FrameOffset;

										PinnedFade->GetMutableFadeRegions()[Index].FadeIn = NewObject<UTransformationFadeFunction>(PinnedFade.Get(), FadeOptionPair.Value.Get(), NAME_None, RF_Transactional);
										PinnedFade->GetFadeRegions()[Index].FadeIn->Duration = Duration;
										PinnedFade->GetFadeRegions()[Index].FadeIn->FrameOffset = Offset;
									}
								}
								else if (InteractionTypeAndIndex.Key == FWaveformTransformationFadeRenderer::EFadeInteractionType::RightClickFadeOut)
								{
									if (PinnedFade->GetFadeRegions()[Index].FadeOut)
									{
										const float Duration = PinnedFade->GetFadeRegions()[Index].FadeOut->Duration;
										const int64 Offset = PinnedFade->GetFadeRegions()[Index].FadeOut->FrameOffset;

										PinnedFade->GetMutableFadeRegions()[Index].FadeOut = NewObject<UTransformationFadeFunction>(PinnedFade.Get(), FadeOptionPair.Value.Get(), NAME_None, RF_Transactional);
										PinnedFade->GetFadeRegions()[Index].FadeOut->Duration = Duration;
										PinnedFade->GetFadeRegions()[Index].FadeOut->FrameOffset = Offset;
									}
								}

								if (GEditor && GEditor->Trans)
								{
									GEditor->EndTransaction();
								}

								PinnedFade->PostEditChange();
								InOnFadeModeChanged.ExecuteIfBound();
							}
						}

						TSharedPtr<SWindow> LockedFadeModeMenuWindow = InWeakFadeModeMenuWindow.Pin();
						if (LockedFadeModeMenuWindow)
						{
							FSlateApplication::Get().RequestDestroyWindow(LockedFadeModeMenuWindow.ToSharedRef());
							
							if (InPopupHandle.IsValid())
							{
								FSlateApplication::Get().OnFocusChanging().Remove(InPopupHandle);
							}

							if (InApplicationActivationStateHandle.IsValid())
							{
								FSlateApplication::Get().OnApplicationActivationStateChanged().Remove(InApplicationActivationStateHandle);
							}
						}

						return FReply::Handled();
					})
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						[
							SNew(SImage)
							.Image(GetFadeModeIcon(FadeOptionPair.Key).GetIcon())
						]
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.Padding(5)
						[
							SNew(STextBlock)
							.Text(FText::FromString(StaticEnum<EWaveEditorTransformationFadeMode>()->GetNameStringByValue(static_cast<int64>(FadeOptionPair.Key))))
						]
					]
			];
	}

	FSlateApplication::Get().AddWindow(MenuWindow);

	// The window does not have a valid position and size until after 1 tick
	// If the new window is off screen, shift it back on screen
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([InFadeModeMenuWindow = FadeModeMenuWindow, LocalCursorPosition, LocalWindowMaxPosition](float DeltaTime)
		{
			if (TSharedPtr<SWindow> LockedFadeModeMenuWindow = InFadeModeMenuWindow.Pin())
			{
				const FVector2D WindowSize = LockedFadeModeMenuWindow->GetSizeInScreen();

				if (WindowSize.X > 0.0 && LocalCursorPosition.X + WindowSize.X > LocalWindowMaxPosition.X)
				{
					check(WindowSize.Y > 0.0);

					const double XPosition = FMath::Max(0.0, LocalCursorPosition.X - WindowSize.X);
					LockedFadeModeMenuWindow->MoveWindowTo(FVector2D(XPosition, LocalCursorPosition.Y));
				}
			}

			return false; // return false to only execute once
		}));
}

const FSlateIcon FWaveformTransformationFadeRenderer::GetFadeModeIcon(const EWaveEditorTransformationFadeMode& FadeMode) const
{
	if (FadeInteractionTypeAndIndex.Key == FWaveformTransformationFadeRenderer::EFadeInteractionType::RightClickFadeIn)
	{
		switch (FadeMode)
		{
		case EWaveEditorTransformationFadeMode::Linear:
			return FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeInLinear");
		case EWaveEditorTransformationFadeMode::Exponential:
			return FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeInExponential");
		case EWaveEditorTransformationFadeMode::Logarithmic:
			return FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeInLogarithmic");
		case EWaveEditorTransformationFadeMode::Sigmoid:
			return FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeInSigmoid");
		default:
			return FSlateIcon();
		}
	}
	else if (FadeInteractionTypeAndIndex.Key == FWaveformTransformationFadeRenderer::EFadeInteractionType::RightClickFadeOut)
	{
		switch (FadeMode)
		{
		case EWaveEditorTransformationFadeMode::Linear:
			return FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeOutLinear");
		case EWaveEditorTransformationFadeMode::Exponential:
			return FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeOutExponential");
		case EWaveEditorTransformationFadeMode::Logarithmic:
			return FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeOutLogarithmic");
		case EWaveEditorTransformationFadeMode::Sigmoid:
			return FSlateIcon(FAudioWidgetsStyle::StyleName, "AudioWidgetsStyle.FadeOutSigmoid");
		default:
			return FSlateIcon();
		}
	}

	return FSlateIcon();
}

TPair<FWaveformTransformationFadeRenderer::EFadeInteractionType, int32> FWaveformTransformationFadeRenderer::GetInteractionTypeAndIndexFromCursorPosition(const FVector2D& InLocalCursorPosition, const FKey MouseButton, const FGeometry& WidgetGeometry) const
{
	for (int32 Index = 0; Index < FadeRegions.Num(); Index++)
	{
		if (MouseButton == EKeys::LeftMouseButton)
		{
			if (FadeRegions[Index].FadeInStartInteractionXRange.Contains(InLocalCursorPosition.X) && FadeRegions[Index].FadeInStartInteractionYRange.Contains(InLocalCursorPosition.Y))
			{
				return TPair<EFadeInteractionType, int32>(EFadeInteractionType::ScrubbingFadeInStart, Index);
			}

			if (FadeRegions[Index].FadeInEndInteractionXRange.Contains(InLocalCursorPosition.X) && FadeRegions[Index].FadeInEndInteractionYRange.Contains(InLocalCursorPosition.Y))
			{
				return TPair<EFadeInteractionType, int32>(EFadeInteractionType::ScrubbingFadeInEnd, Index);
			}

			if (FadeRegions[Index].FadeOutStartInteractionXRange.Contains(InLocalCursorPosition.X) && FadeRegions[Index].FadeOutStartInteractionYRange.Contains(InLocalCursorPosition.Y))
			{
				return TPair<EFadeInteractionType, int32>(EFadeInteractionType::ScrubbingFadeOutStart, Index);
			}

			if (FadeRegions[Index].FadeOutEndInteractionXRange.Contains(InLocalCursorPosition.X) && FadeRegions[Index].FadeOutEndInteractionYRange.Contains(InLocalCursorPosition.Y))
			{
				return TPair<EFadeInteractionType, int32>(EFadeInteractionType::ScrubbingFadeOutEnd, Index);
			}
		}
		else if (MouseButton == EKeys::RightMouseButton)
		{
			if (FadeRegions[Index].FadeInEndInteractionXRange.Contains(InLocalCursorPosition.X) && FadeRegions[Index].FadeInEndInteractionYRange.Contains(InLocalCursorPosition.Y))
			{
				return TPair<EFadeInteractionType, int32>(EFadeInteractionType::RightClickFadeIn, Index);
			}

			if (FadeRegions[Index].FadeOutStartInteractionXRange.Contains(InLocalCursorPosition.X) && FadeRegions[Index].FadeOutStartInteractionYRange.Contains(InLocalCursorPosition.Y))
			{
				return TPair<EFadeInteractionType, int32>(EFadeInteractionType::RightClickFadeOut, Index);
			}
		}
	}

	return TPair<EFadeInteractionType, int32>(EFadeInteractionType::None, 0);
}

FText FWaveformTransformationFadeRenderer::GetPropertyEditedByCurrentInteraction() const
{

	FText OutPropertyName = FText::FromName(NAME_None);

	if (StrongFade->GetFadeRegions().IsValidIndex(FadeInteractionTypeAndIndex.Value))
	{
		check(StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeIn);
		check(StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeOut);

		switch (FadeInteractionTypeAndIndex.Key)
		{
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::None:
			break;
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeInStart:
			OutPropertyName = FText::Format(FText::FromString("{0}::{1}"),
				FText::FromName(GET_MEMBER_NAME_CHECKED(FTransformationFadeFunctionData, FadeIn)),
				FText::FromName(GET_MEMBER_NAME_CHECKED(UTransformationFadeFunction, FrameOffset)));

			break;
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeInEnd:
			OutPropertyName = FText::Format(FText::FromString("{0}::{1}"),
				FText::FromName(GET_MEMBER_NAME_CHECKED(FTransformationFadeFunctionData, FadeIn)),
				FText::FromName(GET_MEMBER_NAME_CHECKED(UTransformationFadeFunction, Duration)));
			
			break;
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeOutStart:
			OutPropertyName = FText::Format(FText::FromString("{0}::{1}"),
				FText::FromName(GET_MEMBER_NAME_CHECKED(FTransformationFadeFunctionData, FadeOut)),
				FText::FromName(GET_MEMBER_NAME_CHECKED(UTransformationFadeFunction, Duration)));
			
			break;
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeOutEnd:
			OutPropertyName = FText::Format(FText::FromString("{0}::{1}"),
				FText::FromName(GET_MEMBER_NAME_CHECKED(FTransformationFadeFunctionData, FadeOut)),
				FText::FromName(GET_MEMBER_NAME_CHECKED(UTransformationFadeFunction, FrameOffset)));
			
			break;
		default:
			break;
		}
	}

	return MoveTemp(OutPropertyName);
}