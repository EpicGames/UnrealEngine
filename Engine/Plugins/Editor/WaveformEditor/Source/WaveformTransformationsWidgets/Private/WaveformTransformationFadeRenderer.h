// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaveformTransformationRendererBase.h"
#include "WaveformTransformationFade.h"
#include "WaveformTransformationTrimFade.h"

#include "Brushes/SlateRoundedBoxBrush.h"

struct FFadeRegion
{
	uint32 FadeInStartX = 0;
	uint32 FadeInEndX = 0;
	uint32 FadeOutStartX = 0;
	uint32 FadeOutEndX = 0;

	TArray<FVector2D> FadeInCurvePoints;
	TArray<FVector2D> FadeOutCurvePoints;

	TRange<float> FadeInStartInteractionXRange = TRange<float>::Inclusive(0.f, 0.f);
	TRange<float> FadeInEndInteractionXRange = TRange<float>::Inclusive(0.f, 0.f);
	TRange<float> FadeInStartInteractionYRange = TRange<float>::Inclusive(0.f, 0.f);
	TRange<float> FadeInEndInteractionYRange = TRange<float>::Inclusive(0.f, 0.f);
	TRange<float> FadeOutStartInteractionXRange = TRange<float>::Inclusive(0.f, 0.f);
	TRange<float> FadeOutEndInteractionXRange = TRange<float>::Inclusive(0.f, 0.f);
	TRange<float> FadeOutStartInteractionYRange = TRange<float>::Inclusive(0.f, 0.f);
	TRange<float> FadeOutEndInteractionYRange = TRange<float>::Inclusive(0.f, 0.f);
};

class FWaveformTransformationFadeRenderer : public FWaveformTransformationRendererBase
{
public:
	FWaveformTransformationFadeRenderer() : RoundedBoxBrush(FLinearColor::Green, 2) {}
	~FWaveformTransformationFadeRenderer();

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseWheel(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;

	virtual void SetWaveformTransformation(TObjectPtr<UWaveformTransformationBase> InTransformation) override;

protected:
	int32 DrawOffsetHandles(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	int32 DrawFadeCurves(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;
	void GenerateFadeCurves(const FGeometry& AllottedGeometry);
	void UpdateInteractionRange(const FGeometry& AllottedGeometry);

	bool IsCursorInFadeInteractionRange(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry) const;

	FText GetPropertyEditedByCurrentInteraction() const;
	FVector2D GetLocalCursorPosition(const FPointerEvent& MouseEvent, const FGeometry& EventGeometry) const;
	double ConvertXRatioToTime(const float InRatio) const;

	TArray<FFadeRegion> FadeRegions;

	double PixelsPerFrame = 0.0;
	const float InteractionHandleSize = 30.0f;
	FVector2D MousePosition;
	int32 TransactionResult = INDEX_NONE;

	enum class EFadeInteractionType : uint8
	{
		None = 0,
		ScrubbingFadeInStart,
		ScrubbingFadeInEnd, 
		ScrubbingFadeOutStart,
		ScrubbingFadeOutEnd,
		RightClickFadeIn,
		RightClickFadeOut
	};

	TPair<EFadeInteractionType, int32> FadeInteractionTypeAndIndex = TPair<EFadeInteractionType, int32>(EFadeInteractionType::None, 0);

	TPair<EFadeInteractionType, int32> GetInteractionTypeAndIndexFromCursorPosition(const FVector2D& InLocalCursorPosition, const FKey MouseButton, const FGeometry& WidgetGeometry) const;
	void SetPropertyValueDependingOnInteractionType(const FPointerEvent& MouseEvent, const FGeometry& WidgetGeometry, const EPropertyChangeType::Type ChangeType = EPropertyChangeType::ValueSet);
	void ShowSelectFadeModeMenuAtCursor(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent);
	const FSlateIcon GetFadeModeIcon(const EWaveEditorTransformationFadeMode& FadeMode) const;

	const FSlateRoundedBoxBrush RoundedBoxBrush;

	TWeakPtr<SWindow> FadeModeMenuWindow = nullptr;

	TStrongObjectPtr<UWaveformTransformationFade> StrongFade = nullptr;

	FDelegateHandle PopupHandle;
	FDelegateHandle ApplicationActivationStateHandle;

	DECLARE_DELEGATE(FOnFadeModeChanged);
	FOnFadeModeChanged OnFadeModeChanged;
};
