// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationMarkersRenderer.h"

#include "Brushes/SlateRoundedBoxBrush.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "IWaveformTransformation.h"
#include "Layout/Geometry.h"
#include "PropertyHandle.h"
#include "Rendering/DrawElements.h"
#include "Sound/SoundWave.h"
#include "Widgets/Input/SButton.h"
#include <Blueprint/WidgetLayoutLibrary.h>

#define LOCTEXT_NAMESPACE "WaveformTransformationMarkerRenderer"

FWaveformTransformationMarkerRenderer::FWaveformTransformationMarkerRenderer()
{
	const UWaveformTransformationsWidgetsSettings* Settings = GetDefault<UWaveformTransformationsWidgetsSettings>();
	check(Settings);
	OnSettingChangedDelegateHandle = Settings->OnSettingChanged().AddLambda([this](const FName& PropertyName, const UWaveformTransformationsWidgetsSettings* Settings)
		{
			check(Settings);

			if (PropertyName.IsNone())
			{
				MarkerColor = Settings->MarkerColor;
				SetLoopColors(Settings->LoopColors);
				LabelTextColor = Settings->LabelTextColor;
				LabelFontSize = Settings->LabelFontSize;

				return;
			}

			if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformTransformationsWidgetsSettings, MarkerColor))
			{
				MarkerColor = Settings->MarkerColor;
			}
			if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformTransformationsWidgetsSettings, LoopColors))
			{
				SetLoopColors(Settings->LoopColors);
			}
			if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformTransformationsWidgetsSettings, LabelTextColor))
			{
				LabelTextColor = Settings->LabelTextColor;
			}
			if (PropertyName == GET_MEMBER_NAME_CHECKED(UWaveformTransformationsWidgetsSettings, LabelFontSize))
			{
				LabelFontSize = Settings->LabelFontSize;
			}
		});

	ApplyWidgetSettings(Settings);
}

FWaveformTransformationMarkerRenderer::~FWaveformTransformationMarkerRenderer()
{
	FSlateApplication::Get().OnFocusChanging().Remove(PopupHandle);
	FSlateApplication::Get().OnApplicationActivationStateChanged().Remove(ApplicationActivationStateHandle);

	if (OnSettingChangedDelegateHandle.IsValid())
	{
		const UWaveformTransformationsWidgetsSettings* Settings = GetDefault<UWaveformTransformationsWidgetsSettings>();
		check(Settings);
		Settings->OnSettingChanged().Remove(OnSettingChangedDelegateHandle);
	}
}

int32 FWaveformTransformationMarkerRenderer::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = DrawMarkerHandles(AllottedGeometry, OutDrawElements, LayerId);

	return LayerId;
}

int32 FWaveformTransformationMarkerRenderer::DrawMarkerHandles(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	TArray<FVector2D> LinePoints;
	LinePoints.SetNumUninitialized(2);

	check(TransformationWaveInfo.NumChannels > 0);
	const float MarkerHeight = AllottedGeometry.Size.Y * InteractionRatioYDelta;
	constexpr float MarkerWidth = InteractionPixelXDelta * 0.8;
	FSlateRoundedBoxBrush RoundedBoxBrush(FLinearColor::Red, 2);
	FSlateFontInfo Font = FAppStyle::GetFontStyle("Regular");
	Font.Size = LabelFontSize;
	
	uint32 ColorIndex = 0;

	if (LoopPoints.Num() > 0)
	{
		check(LoopColors.Num() > 0);
	}
	for (const FSoundWaveCuePoint& LoopRegion : LoopPoints)
	{
		const float LoopRegionPosition = static_cast<float>(LoopRegion.FramePosition) * PixelsPerFrame;
		LinePoints[0] = FVector2D(LoopRegionPosition, 0.f);
		LinePoints[1] = FVector2D(LoopRegionPosition, AllottedGeometry.Size.Y);

		ColorIndex = (ColorIndex + 1) % LoopColors.Num();

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			LoopColors[ColorIndex],
			false
		);


		const float MarkerCenter = LoopRegionPosition;
		const float	HandleStart = MarkerCenter - MarkerWidth / 2;
		const float LoopRegionPixelLength = static_cast<float>(LoopRegion.FrameLength) * PixelsPerFrame;
		FSlateBrush Brush;
		FPaintGeometry HandleGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(MarkerWidth, MarkerHeight), FSlateLayoutTransform(FVector2f(HandleStart, 0)));
		FPaintGeometry LoopBoxGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(LoopRegionPixelLength, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(MarkerCenter, 0)));
		LinePoints[0] = FVector2D(LoopRegionPosition + LoopRegionPixelLength, 0.f);
		LinePoints[1] = FVector2D(LoopRegionPosition + LoopRegionPixelLength, AllottedGeometry.Size.Y);

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			LoopColors[ColorIndex],
			false
		);

		FLinearColor LoopBoxColor = LoopColors[ColorIndex];
		
		//Make LoopBox always half as opaque as the Marker
		LoopBoxColor = LoopColors[ColorIndex];

		float colorMultiplier = 0.5;

		check(MarkersArray.IsValid());
		// Highlight selection
		if (MarkersArray->SelectedCue != INDEX_NONE && MarkersArray->SelectedCue == LoopRegion.CuePointID)
		{
			colorMultiplier = 0.9f;
		}
		
		LoopBoxColor.A *= colorMultiplier;
		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			LoopBoxGeometry,
			&Brush,
			ESlateDrawEffect::None,
			LoopBoxColor
		);

		// If the loop region has no width or too small to draw handles, use the marker to move it
		if (static_cast<float>(LoopRegion.FrameLength) < LoopHandlePixelWidth * 2.f)
		{
			FSlateDrawElement::MakeBox(
				OutDrawElements,
				++LayerId,
				HandleGeometry,
				&RoundedBoxBrush,
				ESlateDrawEffect::None,
				LoopColors[ColorIndex]
			);
		}

		FVector2D TextOffset(MarkerCenter + MarkerWidth, 0);
		FSlateDrawElement::MakeText(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(AllottedGeometry.Size, FSlateLayoutTransform(TextOffset)),
			LoopRegion.Label,
			Font,
			ESlateDrawEffect::None,
			LabelTextColor
		);

		// If we have a loop region highlighted by the mouse, draw the handles
		const bool DrawRegionHandles = (HighlightedCue != INDEX_NONE && HighlightedCue == LoopRegion.CuePointID && static_cast<float>(LoopRegion.FrameLength) > LoopHandlePixelWidth);

		if (DrawRegionHandles)
		{
			FPaintGeometry LeftHandleGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(LoopHandlePixelWidth, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(LoopRegionPosition, 0)));
			FPaintGeometry RightHandleGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(LoopHandlePixelWidth, AllottedGeometry.GetLocalSize().Y), FSlateLayoutTransform(FVector2f(LoopRegionPosition + LoopRegionPixelLength - LoopHandlePixelWidth, 0)));
			const bool bIsLeftHandleHighlighted = IsPositionInLoopHandleArea(MousePosition, AllottedGeometry, LoopRegion, true);
			const bool bIsRightHandleHighlighted = IsPositionInLoopHandleArea(MousePosition, AllottedGeometry, LoopRegion, false);

			FString HighlightHex = TEXT("#0078D7");
			FColor HighlightColor = FColor::FromHex(HighlightHex);
			FLinearColor HighlightLinearColor = FLinearColor::FromSRGBColor(HighlightColor);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				++LayerId,
				LeftHandleGeometry,
				&Brush,
				ESlateDrawEffect::None,
				bIsLeftHandleHighlighted ? HighlightLinearColor : FLinearColor::White
			);

			FSlateDrawElement::MakeBox(
				OutDrawElements,
				++LayerId,
				RightHandleGeometry,
				&Brush,
				ESlateDrawEffect::None,
				bIsRightHandleHighlighted ? HighlightLinearColor : FLinearColor::White
			);
		}
	}

	for (const FSoundWaveCuePoint& Marker : CuePoints)
	{
		const float MarkerPosition = static_cast<float>(Marker.FramePosition) * static_cast<float>(PixelsPerFrame);
		LinePoints[0] = FVector2D(MarkerPosition, 0.f);
		LinePoints[1] = FVector2D(MarkerPosition, AllottedGeometry.Size.Y);

		FSlateDrawElement::MakeLines(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(),
			LinePoints,
			ESlateDrawEffect::None,
			MarkerColor,
			false
		);

		const float	HandleStart = MarkerPosition - MarkerWidth / 2;
		FPaintGeometry HandleGeometry = AllottedGeometry.ToPaintGeometry(FVector2f(MarkerWidth, MarkerHeight), FSlateLayoutTransform(FVector2f(HandleStart, 0)));

		FSlateDrawElement::MakeBox(
			OutDrawElements,
			++LayerId,
			HandleGeometry,
			&RoundedBoxBrush,
			ESlateDrawEffect::None,
			MarkersArray->SelectedCue == Marker.CuePointID ? SelectedMarkerColor : MarkerColor
		);

		FVector2D TextOffset(MarkerPosition + MarkerWidth, 0);
		FSlateDrawElement::MakeText(
			OutDrawElements,
			++LayerId,
			AllottedGeometry.ToPaintGeometry(AllottedGeometry.Size, FSlateLayoutTransform(TextOffset)),
			Marker.Label,
			Font,
			ESlateDrawEffect::None,
			LabelTextColor
		);
	}

	return LayerId;
}

void FWaveformTransformationMarkerRenderer::SetWaveformTransformation(TObjectPtr<UWaveformTransformationBase> InTransformation)
{
	FWaveformTransformationRendererBase::SetWaveformTransformation(InTransformation);
	StrongMarkersTransformation = TStrongObjectPtr<UWaveformTransformationMarkers>(CastChecked<UWaveformTransformationMarkers>(InTransformation.Get()));

	MarkersArray = TStrongObjectPtr<UWaveCueArray>(CastChecked<UWaveCueArray>(StrongMarkersTransformation->Markers));
}

void FWaveformTransformationMarkerRenderer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{ 
	check(TransformationWaveInfo.NumChannels > 0);
	const float NumFrames = static_cast<float>(TransformationWaveInfo.TotalNumSamples) / static_cast<float>(TransformationWaveInfo.NumChannels);
	check(NumFrames > 0);
	PixelsPerFrame = FMath::Max(AllottedGeometry.GetLocalSize().X / NumFrames, UE_SMALL_NUMBER);
	ensure(PixelsPerFrame > 0);
	MarkerInInteractionRange.SetMinLoopSize((LoopHandlePixelWidth / PixelsPerFrame) * 3);

	FVector2D MouseAbsolutePosition = UWidgetLayoutLibrary::GetMousePositionOnPlatform();
	MousePosition = AllottedGeometry.AbsoluteToLocal(MouseAbsolutePosition);

	check(StrongMarkersTransformation);
	check(MarkersArray != nullptr);

	TMap<int32, FSoundWaveCuePoint> CuePointMap;
	CuePointMap.Reserve(MarkersArray->CuesAndLoops.Num());

	bool bIsMarkerInInteractionRangeValid = false;
	bool bFixedDuplicateCuePoints = false;
	for (const FSoundWaveCuePoint& Marker : MarkersArray->CuesAndLoops)
	{
		//Fix duplicate loop region markers for SoundWaves that have not been reimported 
		//after import fix
		ensure(Marker.CuePointID != INDEX_NONE);
		if (!CuePointMap.Contains(Marker.CuePointID))
		{
			CuePointMap.Emplace(Marker.CuePointID, Marker);
		}
		else if (Marker.FrameLength > 0)
		{
			CuePointMap[Marker.CuePointID].FrameLength = Marker.FrameLength;
			CuePointMap[Marker.CuePointID].SetLoopRegion(true);
			bFixedDuplicateCuePoints = true;
		}

		if (&Marker == MarkerInInteractionRange.GetMarker())
		{
			bIsMarkerInInteractionRangeValid = true;
		}

		check(TransformationWaveInfo.NumChannels > 0);
		int64 StartFrameOffset = TransformationWaveInfo.StartFrameOffset / TransformationWaveInfo.NumChannels;
		int64 NumAvailableSamples = static_cast<int64>(TransformationWaveInfo.NumSamplesAvailable) / TransformationWaveInfo.NumChannels;

		// If loop region is no longer within the available frames, revert Loop preview handles 
		if (Marker.CuePointID == MarkersArray->SelectedCue && Marker.IsLoopRegion() &&
			(Marker.FramePosition + Marker.FrameLength < StartFrameOffset || Marker.FramePosition > StartFrameOffset + NumAvailableSamples))
		{
			MarkersArray->Modify(bMarkFileDirty);
			MarkersArray->SelectedCue = INDEX_NONE;
			
			if (StrongMarkersTransformation->ResetLoopPreviewing())
			{
				StrongMarkersTransformation->OnTransformationChanged.ExecuteIfBound(true);
			}

			//End move transaction
			EndTransaction();
		}
	}

	StrongMarkersTransformation->AdjustLoopPreviewIfNotAligned();

	//Invalidate Marker pointer if it has been removed from the array
	if (!bIsMarkerInInteractionRangeValid)
	{
		MarkerInInteractionRange.SetMarkerInInteractionRange(nullptr);
	}

	if (bFixedDuplicateCuePoints && !CuePointMap.IsEmpty())
	{
		BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("RectifyingMarkerArray", "Rectify Marker Array (Fixed duplicate values)"), StrongMarkersTransformation.Get());
		MarkersArray->Modify();
		CuePointMap.GenerateValueArray(MarkersArray->CuesAndLoops);
		EndTransaction();
	}

	CuePoints.Empty();
	LoopPoints.Empty();
	for (TPair<int32, FSoundWaveCuePoint>& Pair : CuePointMap)
	{
		if (Pair.Value.IsLoopRegion())
		{
			LoopPoints.Add(MoveTemp(Pair.Value));
		}
		else
		{
			CuePoints.Add(MoveTemp(Pair.Value));
		}
	}

	CuePointMap.Empty();
}

FReply FWaveformTransformationMarkerRenderer::OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const FVector2D LocalCursorPosition = GetLocalCursorPosition(MouseEvent, MyGeometry);

	MarkerInteractionType = EvaluateInteractionTypeFromCursorPosition(LocalCursorPosition, MyGeometry, MouseEvent.GetEffectingButton());

	if (MarkerInteractionType != EMarkerInteractionType::None)
	{
		return FReply::Handled().CaptureMouse(OwnerWidget.AsShared()).PreventThrottling();
	}

	return FReply::Unhandled();
}

FReply FWaveformTransformationMarkerRenderer::OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	check(StrongMarkersTransformation);

	if (MarkersArray == nullptr)
	{
		return FReply::Unhandled();
	}

	bool bHasTransformationChanged = false;

	if (MarkersArray->SelectedCue == INDEX_NONE)
	{
		bHasTransformationChanged = StrongMarkersTransformation->ResetLoopPreviewing();
	}
	else
	{
		PreviewSelectedLoop();
		bHasTransformationChanged = true;
	}

	if (MarkerInteractionType != EMarkerInteractionType::None)
	{
		bool bMarkDirty = MarkerInteractionType == EMarkerInteractionType::ScrubbingMarkerHandleLeft || MarkerInteractionType == EMarkerInteractionType::ScrubbingMarkerHandleRight;

		StrongMarkersTransformation->Modify(bMarkDirty);
		SetPropertyValueDependingOnInteractionType(MouseEvent, MyGeometry);

		// If we know it was an interaction that started a transaction, end the transaction
		if (MarkerInteractionType == EMarkerInteractionType::ScrubbingMarkerHandleLeft || MarkerInteractionType == EMarkerInteractionType::ScrubbingMarkerHandleRight
			|| MarkerInteractionType == EMarkerInteractionType::LoopHandle || MarkerInteractionType == EMarkerInteractionType::MarkerHandle)
		{
			EndTransaction();
		}

		MarkerInteractionType = EMarkerInteractionType::None;
		
		StrongMarkersTransformation->OnTransformationChanged.ExecuteIfBound(bHasTransformationChanged);
		
		return FReply::Handled().ReleaseMouseCapture();
	}

	if (bHasTransformationChanged)
	{
		StrongMarkersTransformation->OnTransformationChanged.ExecuteIfBound(true);
	}

	return FReply::Unhandled();
}

FReply FWaveformTransformationMarkerRenderer::OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	check(StrongMarkersTransformation);
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && MarkerInteractionType != EMarkerInteractionType::None)
	{
		SetPropertyValueDependingOnInteractionType(MouseEvent, MyGeometry);
		StrongMarkersTransformation->OnTransformationRenderChanged.ExecuteIfBound();

		return FReply::Handled().CaptureMouse(OwnerWidget.AsShared());
	}

	if (MarkersArray == nullptr)
	{
		return FReply::Unhandled();
	}

	// Evaluate highlight state for loop handles
	HighlightedCue = INDEX_NONE;
	for (const FSoundWaveCuePoint& CuePoint : MarkersArray->CuesAndLoops)
	{
		if (CuePoint.IsLoopRegion() && IsPositionInLoopArea(MousePosition, MyGeometry, &CuePoint))
		{
			HighlightedCue = CuePoint.CuePointID;
		}
	}

	return FReply::Unhandled();
}

FCursorReply FWaveformTransformationMarkerRenderer::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	// Grab hand if moving a regular marker
	if (MarkerInteractionType == EMarkerInteractionType::MarkerHandle)
	{
		return FCursorReply::Cursor(EMouseCursor::GrabHandClosed);
	}

	if (MarkersArray == nullptr)
	{
		return FCursorReply::Unhandled();
	}

	// Set resize on loop handle hover, loops are separated as we want highlight interactions to superscede region highlights
	for (const FSoundWaveCuePoint& CuePoint : MarkersArray->CuesAndLoops)
	{
		const bool bIsLeftHandleHighlighted = IsPositionInLoopHandleArea(MousePosition, MyGeometry, CuePoint, true) || 
			(MarkerInteractionType == EMarkerInteractionType::ScrubbingMarkerHandleLeft && HighlightedCue == CuePoint.CuePointID);
		const bool bIsRightHandleHighlighted = IsPositionInLoopHandleArea(MousePosition, MyGeometry, CuePoint, false) || 
			(MarkerInteractionType == EMarkerInteractionType::ScrubbingMarkerHandleRight && HighlightedCue == CuePoint.CuePointID);

		if (CuePoint.IsLoopRegion() && (bIsLeftHandleHighlighted || bIsRightHandleHighlighted))
		{
			return FCursorReply::Cursor(EMouseCursor::ResizeLeftRight);
		}
		else if (!CuePoint.IsLoopRegion() && IsPositionInInteractionRange(CuePoint.FramePosition, MousePosition, MyGeometry))
		{
			return FCursorReply::Cursor(EMouseCursor::GrabHand);
		}
	}

	// Set grab hand on loop region moving
	for (const FSoundWaveCuePoint& CuePoint : MarkersArray->CuesAndLoops)
	{
		if (CuePoint.IsLoopRegion() && IsPositionInLoopArea(MousePosition, MyGeometry, &CuePoint))
		{
			return FCursorReply::Cursor(EMouseCursor::GrabHand);
		}
	}

	return FCursorReply::Unhandled();
}

FReply FWaveformTransformationMarkerRenderer::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return FReply::Handled();
}

void FWaveformTransformationMarkerRenderer::ApplyWidgetSettings(const UWaveformTransformationsWidgetsSettings* Settings)
{
	check(Settings)

	MarkerColor = Settings->MarkerColor;
	SetLoopColors(Settings->LoopColors);
	LabelTextColor = Settings->LabelTextColor;
	LabelFontSize = Settings->LabelFontSize;
}

void FWaveformTransformationMarkerRenderer::SetLoopColors(const TArray<FLinearColor>& InColors)
{
	LoopColors = InColors;
	if (LoopColors.Num() < 1)
	{
		LoopColors = WaveformTransformationWidgetSharedDefaults::DefaultLoopColors;
	}
}

bool FWaveformTransformationMarkerRenderer::IsPositionInInteractionRange(const int64 InFramePosition, const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry) const
{
	TRange InteractionRange(TRange<float>::Inclusive(static_cast<float>(InFramePosition) * PixelsPerFrame - InteractionPixelXDelta, static_cast<float>(InFramePosition) * PixelsPerFrame + InteractionPixelXDelta));
	if (InteractionRange.Contains(InLocalCursorPosition.X) &&
		InLocalCursorPosition.Y < WidgetGeometry.GetLocalSize().Y * InteractionRatioYDelta)
	{
		return true;
	}
	
	return false;
}

bool FWaveformTransformationMarkerRenderer::IsPositionInLoopArea(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry, const FSoundWaveCuePoint* CueMarker, bool IncludeHandleArea) const
{
	check(TransformationWaveInfo.NumChannels > 0);
	check(CueMarker != nullptr);
	ensure(LoopHandlePixelWidth > 0);

	float LoopRegionPosition = static_cast<float>(CueMarker->FramePosition) * static_cast<float>(PixelsPerFrame);
	float LoopRegionPixelLength = static_cast<float>(CueMarker->FrameLength) * static_cast<float>(PixelsPerFrame);

	// If we want the handles to be included for the area check, as having the mouse in the handle area has different behaviour to the total loop area
	if (!IncludeHandleArea)
	{
		LoopRegionPosition += LoopHandlePixelWidth;
		LoopRegionPixelLength -= LoopHandlePixelWidth * 2;
	}

	check(LoopRegionPixelLength >= 0);

	if (InLocalCursorPosition.X >= LoopRegionPosition && InLocalCursorPosition.X <= LoopRegionPosition + LoopRegionPixelLength &&
		InLocalCursorPosition.Y <= WidgetGeometry.GetLocalSize().Y && InLocalCursorPosition.Y >= 0)
	{
		return true;
	}

	return false;
}

bool FWaveformTransformationMarkerRenderer::IsPositionInLoopHandleArea(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry, const FSoundWaveCuePoint& CueMarker, bool IsLeftHandle) const
{
	check(TransformationWaveInfo.NumChannels > 0);
	ensure(LoopHandlePixelWidth > 0);

	float LoopRegionPosition = IsLeftHandle ?
		static_cast<float>(CueMarker.FramePosition) * static_cast<float>(PixelsPerFrame) :
		(static_cast<float>(CueMarker.FramePosition + CueMarker.FrameLength) * static_cast<float>(PixelsPerFrame)) - LoopHandlePixelWidth;

	// Check that only the handle within the loop area is highlighted
	if (static_cast<float>(InLocalCursorPosition.X) > LoopRegionPosition && static_cast<float>(InLocalCursorPosition.X) < LoopRegionPosition + LoopHandlePixelWidth &&
		static_cast<float>(InLocalCursorPosition.Y) < WidgetGeometry.GetLocalSize().Y && InLocalCursorPosition.Y > 0.0)
	{
		return true;
	}

	return false;
}

//Returns true if there is a marker in interaction range
bool FWaveformTransformationMarkerRenderer::SetMarkerInInteractionRange(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry)
{
	check(TransformationWaveInfo.NumChannels > 0);
	check(TransformationWaveInfo.SampleRate > 0);
	ensure(LoopHandlePixelWidth > 0);
	check(MarkersArray != nullptr);

	InteractionRanges.Reset(MarkersArray->CuesAndLoops.Num());
	
	for (FSoundWaveCuePoint& CuePoint : MarkersArray->CuesAndLoops)
	{
		if (IsPositionInInteractionRange(CuePoint.FramePosition, InLocalCursorPosition, WidgetGeometry))
		{
			MarkerInInteractionRange.SetMarkerInInteractionRange(&CuePoint);
			MarkersArray->SelectedCue = CuePoint.CuePointID;
			
			return true;
		}

		if (CuePoint.FrameLength == 0)
		{
			continue;
		}

		// Check highlights of handles independantly first over loop area
		if (IsPositionInLoopHandleArea(InLocalCursorPosition, WidgetGeometry, CuePoint, true) || IsPositionInLoopHandleArea(InLocalCursorPosition, WidgetGeometry, CuePoint, false))
		{
			MarkerInInteractionRange.SetMarkerInInteractionRange(&CuePoint, true);

			return true;
		}
	}

	check(PixelsPerFrame > 0);

	// Evaluate loop area if no handles are selected
	for (FSoundWaveCuePoint& CuePoint : MarkersArray->CuesAndLoops)
	{
		if (CuePoint.IsLoopRegion() && IsPositionInLoopArea(InLocalCursorPosition, WidgetGeometry, &CuePoint))
		{
			const int64 LoopRegionPosition = static_cast<int64>(InLocalCursorPosition.X / PixelsPerFrame - static_cast<double>(CuePoint.FramePosition));

			MarkerInInteractionRange.SetMarkerInInteractionRange(&CuePoint, false, LoopRegionPosition);
			MarkersArray->SelectedCue = CuePoint.CuePointID;

			SetActiveLoopRegion(MarkersArray->SelectedCue);

			return true;
		}
	}

	// MarkerPtr is set to null if there are no markers in range to handle cases where the mouse is not over any markers
	MarkerInInteractionRange.SetMarkerInInteractionRange(nullptr);

	if (MarkersArray->SelectedCue == INDEX_NONE)
	{
		StrongMarkersTransformation->ResetLoopPreviewing();
	}
	else
	{
		PreviewSelectedLoop();
	}
	
	return false;
}

bool FWaveformTransformationMarkerRenderer::SelectLoopArea(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry)
{
	check(TransformationWaveInfo.NumChannels > 0);
	ensure(LoopHandlePixelWidth > 0);

	for (FSoundWaveCuePoint& LoopRegion : LoopPoints)
	{
		// Check mouse position bounds
		if (LoopRegion.IsLoopRegion() && IsPositionInLoopArea(InLocalCursorPosition, WidgetGeometry, &LoopRegion))
		{
			SetActiveLoopRegion(LoopRegion.CuePointID);

			return true;
		}
	}

	MarkersArray->SelectedCue = INDEX_NONE;

	return false;
}

void FWaveformTransformationMarkerRenderer::SetActiveLoopRegion(int32 CuePointID)
{
	check(StrongMarkersTransformation.IsValid());
	check(MarkersArray != nullptr);

	MarkersArray->Modify(false);
	MarkersArray->SelectedCue = CuePointID;

	check(TransformationWaveInfo.SampleRate > 0);
	// Revert Loop preview handles
	if (CuePointID == INDEX_NONE || StrongMarkersTransformation->GetSelectedMarker() == nullptr || !StrongMarkersTransformation->GetSelectedMarker()->IsLoopRegion())
	{
		SetIsPreviewingLoopRegion(0.0, -1.0, false);
	}
	else
	{
		PreviewSelectedLoop();
	}

	EndTransaction();
}

void FWaveformTransformationMarkerRenderer::PreviewSelectedLoop()
{
	check(TransformationWaveInfo.SampleRate > 0);
	check(TransformationWaveInfo.NumChannels > 0);

	const int64 StartFrameOffset = TransformationWaveInfo.StartFrameOffset / TransformationWaveInfo.NumChannels;
	const int64 EndFramePosition = StartFrameOffset + TransformationWaveInfo.NumSamplesAvailable / TransformationWaveInfo.NumChannels;

	for (FSoundWaveCuePoint CuePoint : MarkersArray->CuesAndLoops)
	{
		const int64 LoopEndFramePosition = CuePoint.FramePosition + CuePoint.FrameLength;
		
		// Only preview a loop if part of it is within the available frames (accounting for TrimFades and other transformations)
		if (CuePoint.CuePointID == MarkersArray->SelectedCue && CuePoint.IsLoopRegion() &&
			LoopEndFramePosition >= StartFrameOffset && CuePoint.FramePosition <= EndFramePosition)
		{
			check(CuePoint.FrameLength > 0);

			// Set Loop preview handles
			const float StartLoopPosInSeconds = static_cast<float>(FMath::Max(0, CuePoint.FramePosition - StartFrameOffset)) / TransformationWaveInfo.SampleRate;
			const float EndLoopPosInSeconds = static_cast<float>(LoopEndFramePosition - StartFrameOffset) / TransformationWaveInfo.SampleRate;

			check(StartLoopPosInSeconds >= 0);

			SetIsPreviewingLoopRegion(static_cast<double>(StartLoopPosInSeconds), static_cast<double>(EndLoopPosInSeconds), true);
		}
	}
}

void FWaveformTransformationMarkerRenderer::SetIsPreviewingLoopRegion(double InStartTime, double InEndTime, bool bIsPreviewing)
{
	check(StrongMarkersTransformation);
	// Stop any current loop previewing so StartLoopTime is never greater than EndLoopTime (Other than when EndLoopTime is invalid)
	StrongMarkersTransformation->ResetLoopPreviewing();
	StrongMarkersTransformation->SetLoopPreviewing();
}

FVector2D FWaveformTransformationMarkerRenderer::GetLocalCursorPosition(const FPointerEvent& MouseEvent, const FGeometry& EventGeometry)
{
	const FVector2D ScreenSpacePosition = MouseEvent.GetScreenSpacePosition();
	
	return  EventGeometry.AbsoluteToLocal(ScreenSpacePosition);
}

FWaveformTransformationMarkerRenderer::EMarkerInteractionType FWaveformTransformationMarkerRenderer::EvaluateInteractionTypeFromCursorPosition(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry, const FKey MouseButton)
{
	check(StrongMarkersTransformation);
	if (MouseButton == EKeys::RightMouseButton)
	{
		if (SetMarkerInInteractionRange(InLocalCursorPosition, WidgetGeometry) && MarkerInInteractionRange.GetMarker() != nullptr)
		{
			return EMarkerInteractionType::RightClickMarker;
		}
	}

	if (MouseButton != EKeys::LeftMouseButton)
	{
		return EMarkerInteractionType::None;
	}

	if (MarkersArray == nullptr)
	{
		return EMarkerInteractionType::None;
	}

	for (FSoundWaveCuePoint CuePoint : MarkersArray->CuesAndLoops)
	{
		if (!CuePoint.IsLoopRegion())
		{
			continue;
		}			
			
		if (IsPositionInLoopHandleArea(InLocalCursorPosition, WidgetGeometry, CuePoint, true))
		{
			SetMarkerInInteractionRange(InLocalCursorPosition, WidgetGeometry);
			BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("ResizingMarker", "Resizing Marker"), StrongMarkersTransformation.Get());
			MarkersArray->Modify(bMarkFileDirty);
			return EMarkerInteractionType::ScrubbingMarkerHandleLeft;
		}

		if (IsPositionInLoopHandleArea(InLocalCursorPosition, WidgetGeometry, CuePoint, false))
		{
			SetMarkerInInteractionRange(InLocalCursorPosition, WidgetGeometry);
			BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("ResizingMarker", "Resizing Marker"), StrongMarkersTransformation.Get());
			MarkersArray->Modify(bMarkFileDirty);
			return EMarkerInteractionType::ScrubbingMarkerHandleRight;
		}
	}

	if (SetMarkerInInteractionRange(InLocalCursorPosition, WidgetGeometry))
	{
		BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("MoveMarker", "Move Marker"), StrongMarkersTransformation.Get());
		MarkersArray->Modify(bMarkFileDirty);
		return EMarkerInteractionType::MarkerHandle;
	}
	
	if (SelectLoopArea(InLocalCursorPosition, WidgetGeometry))
	{
		SetMarkerInInteractionRange(InLocalCursorPosition, WidgetGeometry);
		BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("MoveLoop", "Move Loop"), StrongMarkersTransformation.Get());
		MarkersArray->Modify(bMarkFileDirty);
		return EMarkerInteractionType::LoopHandle;
	}
	else
	{
		if (StrongMarkersTransformation->IsLoopRegionActive())
		{
			return EMarkerInteractionType::DeselectRegion;
		}
	}

	return EMarkerInteractionType::None;
}

void FWaveformTransformationMarkerRenderer::SetPropertyValueDependingOnInteractionType(const FPointerEvent& MouseEvent, const FGeometry& WidgetGeometry)
{
	ensure(PixelsPerFrame > 0);

	const FVector2D LocalCursorPosition = GetLocalCursorPosition(MouseEvent, WidgetGeometry);
	const float LocalCursorXClamped = FMath::Clamp(LocalCursorPosition.X, 0.f, WidgetGeometry.GetLocalSize().X);

	switch (MarkerInteractionType)
	{
	case EMarkerInteractionType::None:
		break;
	case EMarkerInteractionType::MarkerHandle:
		{
			check(PixelsPerFrame > 0);
			MarkerInInteractionRange.SetMarkerPosition(LocalCursorXClamped / PixelsPerFrame, false);
		}
		break;
	case EMarkerInteractionType::ScrubbingMarkerHandleRight:
		{
			if (MarkerInInteractionRange.GetMarker())
			{
				check(PixelsPerFrame > 0);
				check(TransformationWaveInfo.NumChannels > 0);
				MarkerInInteractionRange.SetMarkerPosition((LocalCursorXClamped / PixelsPerFrame), false);
				HighlightedCue = MarkerInInteractionRange.GetMarker()->CuePointID;
			}
		}
		break;
	case EMarkerInteractionType::ScrubbingMarkerHandleLeft:
	{
		if (MarkerInInteractionRange.GetMarker())
		{
			check(PixelsPerFrame > 0);
			check(TransformationWaveInfo.NumChannels > 0);
			MarkerInInteractionRange.SetMarkerPosition((LocalCursorXClamped / PixelsPerFrame), true);
			HighlightedCue = MarkerInInteractionRange.GetMarker()->CuePointID;
		}
	}
		break;
	case EMarkerInteractionType::LoopHandle:
	{
		if (MarkerInInteractionRange.GetMarker())
		{
			check(PixelsPerFrame > 0);
			MarkerInInteractionRange.SetMarkerPosition(LocalCursorXClamped / PixelsPerFrame, true);
		}
	}
		break;
	case EMarkerInteractionType::RightClickMarker:
	{
		ShowConvertToLoopAtMarker(WidgetGeometry, MouseEvent, MarkerInInteractionRange.GetMarker());
	}
		break;
	}
}

void FWaveformTransformationMarkerRenderer::ShowConvertToLoopAtMarker(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, FSoundWaveCuePoint* CueToModify)
{
	check(CueToModify);
	check(StrongMarkersTransformation);
	check(MarkersArray != nullptr);
	const FVector2D LocalWindowMaxPosition = MyGeometry.GetAbsolutePosition() + MyGeometry.GetAbsoluteSize();
	const FVector2D LocalCursorPosition = MouseEvent.GetScreenSpacePosition();

	TSharedRef<SVerticalBox> MenuContent = SNew(SVerticalBox);

	if (TSharedPtr<SWindow> LockedLoopMenuWindow = CreateLoopMenuWindow.Pin())
	{
		FSlateApplication::Get().RequestDestroyWindow(LockedLoopMenuWindow.ToSharedRef());
		FSlateApplication::Get().OnFocusChanging().Remove(PopupHandle);
		FSlateApplication::Get().OnApplicationActivationStateChanged().Remove(ApplicationActivationStateHandle);
	}

	TSharedRef<SWindow> MenuWindow = SNew(SWindow)
		.AutoCenter(EAutoCenter::None)
		.Type(EWindowType::Menu)
		.ScreenPosition(LocalCursorPosition)
		.SizingRule(ESizingRule::Autosized)
		.SupportsTransparency(EWindowTransparency::None)
		.IsPopupWindow(true)
		.CreateTitleBar(false)
		[
			MenuContent
		];

	CreateLoopMenuWindow = MenuWindow;
	TWeakObjectPtr<UWaveformTransformationMarkers> WeakMarkersTransformation = StrongMarkersTransformation.Get();

	MenuContent->AddSlot()
		.AutoHeight()
		[
			SNew(SButton)
				.Text(LOCTEXT("WaveformTransformationMarkerRendererToggleLoop", "Toggle between Marker - Loop Region"))
				.OnClicked_Lambda([this, InCreateLoopMenuWindow = CreateLoopMenuWindow, WeakMarkersTransformation, CueToModify, MinLoopSize = MarkerInInteractionRange.GetMinLoopSize()]() -> FReply
					{
						if (TStrongObjectPtr<UWaveformTransformationMarkers> LockedMarkers = WeakMarkersTransformation.Pin())
						{
							if (GEditor && GEditor->Trans)
							{
								GEditor->BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("ConvertMarkerToLoop", "Convert Marker To Loop"), LockedMarkers.Get());
							}

							LockedMarkers->Markers->Modify(bMarkFileDirty);

							if (CueToModify->FrameLength < MinLoopSize)
							{
								CueToModify->FrameLength = MinLoopSize;
							}

							LockedMarkers->Markers->EnableLoopRegion(CueToModify, !CueToModify->IsLoopRegion());

							LockedMarkers->Markers->SelectedCue = CueToModify->CuePointID;

							if (StrongMarkersTransformation != nullptr)
							{
								SetActiveLoopRegion(LockedMarkers->Markers->SelectedCue);

								constexpr bool bMarkDirty = true;
								StrongMarkersTransformation->OnTransformationChanged.ExecuteIfBound(bMarkDirty);
							}
						}

						if (GEditor && GEditor->Trans)
						{
							GEditor->EndTransaction();
						}

						if (TSharedPtr<SWindow> LockedFadeModeMenuWindow = InCreateLoopMenuWindow.Pin())
						{
							FSlateApplication::Get().RequestDestroyWindow(LockedFadeModeMenuWindow.ToSharedRef());
						}
						return FReply::Handled();
					})
		];

	MenuContent->AddSlot()
		.AutoHeight()
		[
			SNew(SButton)
				.Text(LOCTEXT("WaveformTransformationMarkerRendererDelete", "Delete"))
				.OnClicked_Lambda([this, CueToModify]() -> FReply
					{
						BeginTransaction(TEXT("PropertyEditor"), LOCTEXT("Delete", "Delete"), nullptr);

						MarkersArray->Modify();
						if (CueToModify->IsLoopRegion())
						{
							SetIsPreviewingLoopRegion(0.0, -1.0, false);
						}

						for (int32 i = 0; i < MarkersArray->CuesAndLoops.Num(); i++)
						{
							if (CueToModify->CuePointID == MarkersArray->CuesAndLoops[i].CuePointID)
							{
								MarkersArray->CuesAndLoops.RemoveAt(i);
							}
						}

						MarkersArray->SelectedCue = INDEX_NONE;

						constexpr bool bMarkDirty = true;
						StrongMarkersTransformation->OnTransformationChanged.ExecuteIfBound(bMarkDirty);

						EndTransaction();

						if (TSharedPtr<SWindow> LockedFadeModeMenuWindow = CreateLoopMenuWindow.Pin())
						{
							FSlateApplication::Get().RequestDestroyWindow(LockedFadeModeMenuWindow.ToSharedRef());
						}
						return FReply::Handled();
					})
		];

	FSlateApplication::Get().AddWindow(MenuWindow);

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, InCreateLoopMenuWindow = CreateLoopMenuWindow, LocalCursorPosition, LocalWindowMaxPosition](float DeltaTime)
		{
			if (TSharedPtr<SWindow> LockedLoopMenuWindow = InCreateLoopMenuWindow.Pin())
			{
				const FVector2D WindowSize = LockedLoopMenuWindow->GetSizeInScreen();

				if (WindowSize.X > 0.0 && LocalCursorPosition.X + WindowSize.X > LocalWindowMaxPosition.X)
				{
					check(WindowSize.Y > 0.0);

					const double XPosition = FMath::Max(0.0, LocalCursorPosition.X - WindowSize.X);
					LockedLoopMenuWindow->MoveWindowTo(FVector2D(XPosition, LocalCursorPosition.Y));
				}

				const FVector2D SizeScalar(1.1f, 0.8f); // Computed size is too skinny and too tall so we apply this scalar
				FVector2D ComputedSize = LockedLoopMenuWindow->ComputeWindowSizeForContent(LockedLoopMenuWindow->GetContent()->GetDesiredSize());
				ComputedSize *= SizeScalar;
				LockedLoopMenuWindow->Resize(ComputedSize);
			}

			return false; // return false to only execute once
		}));

	FSlateApplication::Get().OnFocusChanging().Remove(PopupHandle);
	FSlateApplication::Get().OnApplicationActivationStateChanged().Remove(ApplicationActivationStateHandle);

	if (TSharedPtr<SWindow> LockedLoopMenuWindow = CreateLoopMenuWindow.Pin())
		ApplicationActivationStateHandle = FSlateApplication::Get().OnApplicationActivationStateChanged().AddLambda([InCreateLoopMenuWindow = CreateLoopMenuWindow](bool isActive)
			{
				if (!isActive)
				{
					if (TSharedPtr<SWindow> LockedLoopMenuWindow = InCreateLoopMenuWindow.Pin())
					{
						FSlateApplication::Get().RequestDestroyWindow(LockedLoopMenuWindow.ToSharedRef());
					}
				}
			});

	// If focus is lost on the popup, destroy it to prevent popups hanging around
	PopupHandle = FSlateApplication::Get().OnFocusChanging().AddLambda([this](const FFocusEvent& FocusEvent, const FWeakWidgetPath& WeakWidgetPath
		, const TSharedPtr<SWidget>& OldWidget, const FWidgetPath& WidgetPath, const TSharedPtr<SWidget>& NewWidget)
		{
			if (CreateLoopMenuWindow != nullptr && CreateLoopMenuWindow.IsValid())
			{
				if (TSharedPtr<SWindow> LockedLoopMenuWindow = CreateLoopMenuWindow.Pin())
				{
					if (OldWidget && !OldWidget->IsHovered() && LockedLoopMenuWindow == OldWidget)
					{
						FSlateApplication::Get().RequestDestroyWindow(LockedLoopMenuWindow.ToSharedRef());
					}
				}
			}
		});
}

void FWaveformTransformationMarkerRenderer::FMarkerInInteractionRange::SetMarkerInInteractionRange(FSoundWaveCuePoint* InMarkerPtr, const bool InIsLoopRegionEndMarker, int64 GrabXOffset)
{
	MarkerPtr = InMarkerPtr;
	IsLoopRegionEndMarker = InIsLoopRegionEndMarker;
	GrabOffset = GrabXOffset;
}

void FWaveformTransformationMarkerRenderer::FMarkerInInteractionRange::SetMarkerPosition(const float InPosition, bool bIsLeft)
{
	check(MarkerPtr);

	if (IsLoopRegionEndMarker)
	{
		//FMath::Max prevents the user from collapsing the loop
		if (bIsLeft)
		{
			int64 MoveDifference = MarkerPtr->FramePosition - static_cast<int64>(InPosition);
			int64 NewFrameLength = FMath::Max(MinLoopSize, MarkerPtr->FrameLength + MoveDifference);
			
			MarkerPtr->FramePosition = static_cast<int64>(InPosition);
			MarkerPtr->FrameLength = NewFrameLength;
		}
		else
		{
			MarkerPtr->FrameLength = FMath::Max(static_cast<int64>(InPosition), MarkerPtr->FramePosition + MinLoopSize) - MarkerPtr->FramePosition;
		}
	}
	else
	{
		MarkerPtr->FramePosition = FMath::Max(0, static_cast<int64>(InPosition) - GrabOffset);
	}
}

#undef LOCTEXT_NAMESPACE
