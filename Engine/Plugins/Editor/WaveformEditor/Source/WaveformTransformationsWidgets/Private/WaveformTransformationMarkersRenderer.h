// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWaveformTransformation.h"
#include "Sound/SoundWave.h"
#include "WaveformTransformationMarkers.h"
#include "WaveformTransformationRendererBase.h"
#include "WaveformTransformationsWidgetsSettings.h"

class FWaveformTransformationMarkerRenderer : public FWaveformTransformationRendererBase
{
public:
	FWaveformTransformationMarkerRenderer();
	~FWaveformTransformationMarkerRenderer();

	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	virtual void SetWaveformTransformation(TObjectPtr<UWaveformTransformationBase> InTransformation) override;

private:
	int32 DrawMarkerHandles(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	void ApplyWidgetSettings(const UWaveformTransformationsWidgetsSettings* Settings);

	void SetLoopColors(const TArray<FLinearColor>& InColors);

	bool IsPositionInInteractionRange(const int64 InFramePosition, const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry) const;
	bool IsPositionInLoopArea(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry, const FSoundWaveCuePoint* CueMarker, bool IncludeHandleArea = true) const;
	bool IsPositionInLoopHandleArea(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry, const FSoundWaveCuePoint& CueMarker, bool IsLeftHandle) const;
	bool SetMarkerInInteractionRange(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry);
	bool SelectLoopArea(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry);
	void SetActiveLoopRegion(int32 CuePointID);
	void PreviewSelectedLoop();
	void SetIsPreviewingLoopRegion(double InStartTime, double InEndTime, bool bIsPreviewing);
	static FVector2D GetLocalCursorPosition(const FPointerEvent& MouseEvent, const FGeometry& EventGeometry);

	TArray<TRange<float>> InteractionRanges;

	enum class EMarkerInteractionType : uint8
	{
		None = 0,
		ScrubbingMarkerHandleRight,
		ScrubbingMarkerHandleLeft,
		LoopHandle,
		MarkerHandle,
		RightClickMarker,
		DeselectRegion,
	} MarkerInteractionType = EMarkerInteractionType::None;

	class FMarkerInInteractionRange
	{
	public:
		void SetMarkerInInteractionRange(FSoundWaveCuePoint* InMarkerPtr, const bool InIsLoopRegionEndMarker = false, int64 GrabXOffset = 0);
		FSoundWaveCuePoint* GetMarker() const { return MarkerPtr; }
		void SetMarkerPosition(const float InPosition, bool bIsLeft);
		void SetMinLoopSize(int64 value) { MinLoopSize = FMath::Max(UWaveCueArray::MinLoopSize, value); }
		const int64 GetMinLoopSize() const { return MinLoopSize; }
		
	private:
		FSoundWaveCuePoint* MarkerPtr = nullptr;
		bool IsLoopRegionEndMarker = false;
		int64 GrabOffset = 0;
		int64 MinLoopSize = UWaveCueArray::MinLoopSize;
	} MarkerInInteractionRange;

	EMarkerInteractionType EvaluateInteractionTypeFromCursorPosition(const FVector2D& InLocalCursorPosition, const FGeometry& WidgetGeometry, const FKey MouseButton);
	void SetPropertyValueDependingOnInteractionType(const FPointerEvent& MouseEvent, const FGeometry& WidgetGeometry);

	void ShowConvertToLoopAtMarker(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, FSoundWaveCuePoint* CueToModify);

	TArray<FSoundWaveCuePoint> CuePoints;
	TArray<FSoundWaveCuePoint> LoopPoints;

	TStrongObjectPtr<UWaveCueArray> MarkersArray;

	double PixelsPerFrame = 0.0;

	// Ensures handles are always visible to a user and we can avoid overlapping handle regions when loop regions are too small, in future consider dynamic relative size based on window size
	static constexpr float LoopHandlePixelWidth = 7.0f; 

	FVector2D MousePosition;
	FSoundWaveCuePoint* MarkerToModify = nullptr;
	int32 HighlightedCue = INDEX_NONE;

	FLinearColor MarkerColor = WaveformTransformationWidgetSharedDefaults::DefaultMarkerColor;
	FLinearColor SelectedMarkerColor = WaveformTransformationWidgetSharedDefaults::DefaultSelectedMarkerColor;
	TArray<FLinearColor> LoopColors = WaveformTransformationWidgetSharedDefaults::DefaultLoopColors;
	FLinearColor LabelTextColor = WaveformTransformationWidgetSharedDefaults::DefaultLabelTextColor;
	float LabelFontSize = WaveformTransformationWidgetSharedDefaults::DefaultLabelFontSize;

	TStrongObjectPtr<UWaveformTransformationMarkers> StrongMarkersTransformation = nullptr;
	static const bool bMarkFileDirty = false;

	FDelegateHandle PopupHandle;
	FDelegateHandle ApplicationActivationStateHandle;
	FDelegateHandle OnSettingChangedDelegateHandle;

	TWeakPtr<SWindow> CreateLoopMenuWindow;
};
