// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationRendererBase.h"

#include "DetailLayoutBuilder.h"
#include "DetailCategoryBuilder.h"
#include "Editor.h"
#include "Input/Reply.h"
#include "Sound/SoundWave.h"

int32 FWaveformTransformationRendererBase::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	return LayerId;
}

FReply FWaveformTransformationRendererBase::OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

FReply FWaveformTransformationRendererBase::OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

FReply FWaveformTransformationRendererBase::OnMouseButtonDoubleClick(SWidget& OwnerWidget, const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	return FReply::Unhandled();
}

FReply FWaveformTransformationRendererBase::OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

FReply FWaveformTransformationRendererBase::OnMouseWheel(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	return FReply::Unhandled();
}

FCursorReply FWaveformTransformationRendererBase::OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const
{
	return FCursorReply::Unhandled();
}

FReply FWaveformTransformationRendererBase::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return FReply::Unhandled();
}

void FWaveformTransformationRendererBase::SetTransformationWaveInfo(const FWaveformTransformationRenderInfo& InWaveInfo)
{
	TransformationWaveInfo = InWaveInfo;
}

void FWaveformTransformationRendererBase::SetPropertyHandles(const TArray<TSharedRef<IPropertyHandle>>& InPropertyHandles)
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	CachedPropertyHandles = InPropertyHandles;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TSharedPtr<IPropertyHandle> FWaveformTransformationRendererBase::GetPropertyHandle(const FName& PropertyName) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<IPropertyHandle> HandlePtr = nullptr;

	for (const TSharedRef<IPropertyHandle>& PropertyHandle : CachedPropertyHandles)
	{
		if (PropertyHandle.Get().GetProperty()->GetFName() == PropertyName)
		{
			check(PropertyHandle.Get().IsValidHandle());
			return PropertyHandle;
		}
	}

	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	check(HandlePtr)
	return HandlePtr;
}

int32 FWaveformTransformationRendererBase::BeginTransaction(const TCHAR* TransactionContext, const FText& Description, UObject* PrimaryObject)
{
	if (GEditor && GEditor->Trans)
	{
		return GEditor->BeginTransaction(TransactionContext, Description, PrimaryObject);
	}

	return INDEX_NONE;
}

int32 FWaveformTransformationRendererBase::EndTransaction()
{
	return GEditor->EndTransaction();
}
