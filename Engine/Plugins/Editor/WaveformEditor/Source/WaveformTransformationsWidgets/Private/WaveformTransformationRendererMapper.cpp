// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationRendererMapper.h"

FWaveformTransformationRendererMapper& FWaveformTransformationRendererMapper::Get()
{
	check(Instance);
	return *Instance;
}

void FWaveformTransformationRendererMapper::Init()
{
	if (Instance == nullptr)
	{
		Instance = MakeUnique<FWaveformTransformationRendererMapper>();
	}
}

void FWaveformTransformationRendererMapper::UnregisterRenderer(const UClass* SupportedTransformation)
{
	check(Instance)
	Instance->TransformationsTypeMap.FindAndRemoveChecked(SupportedTransformation);
}

FWaveformTransformRendererInstantiator* FWaveformTransformationRendererMapper::GetRenderer(const UClass* WaveformTransformationClass)
{
	check(Instance)
	return Instance->TransformationsTypeMap.Find(WaveformTransformationClass);
}

TUniquePtr<FWaveformTransformationRendererMapper> FWaveformTransformationRendererMapper::Instance;
