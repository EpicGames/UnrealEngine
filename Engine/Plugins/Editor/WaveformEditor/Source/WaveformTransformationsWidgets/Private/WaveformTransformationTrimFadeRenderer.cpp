// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationTrimFadeRenderer.h"

#include "Blueprint/WidgetLayoutLibrary.h"
#include "Editor.h"
#include "Framework/Application/SlateApplication.h"
#include "WaveformTransformationLog.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/STextBlock.h"

void FWaveformTransformationTrimFadeRenderer::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	check(TransformationWaveInfo.NumChannels > 0);
	check(StrongFade);
	check(StrongTrimFade);

	const float NumFrames = TransformationWaveInfo.TotalNumSamples / TransformationWaveInfo.NumChannels;

	check(NumFrames > 0);
	PixelsPerFrame = FMath::Max(AllottedGeometry.GetLocalSize().X / NumFrames, 0.0);

	const int32 NumFadeRegions = StrongFade->GetFadeRegionsNum();
	if (NumFadeRegions != FadeRegions.Num())
	{
		FadeRegions.SetNum(NumFadeRegions);
	}

	check(NumFadeRegions == FadeRegions.Num());

	for (int32 Index = 0; Index < NumFadeRegions; Index++)
	{
		check(StrongFade->GetFadeRegions()[Index].FadeIn);
		check(StrongFade->GetFadeRegions()[Index].FadeOut);

		const double FirstFrame = FMath::Clamp((StrongTrimFade->StartTime * TransformationWaveInfo.SampleRate), 0.f, NumFrames);
		const double EndFrame = FMath::Clamp((StrongTrimFade->EndTime * TransformationWaveInfo.SampleRate), FirstFrame, NumFrames);

		FadeRegions[Index].FadeInStartX = static_cast<uint32>(FMath::Clamp((FirstFrame * PixelsPerFrame), 0, UINT32_MAX));
		FadeRegions[Index].FadeOutEndX = static_cast<uint32>(FMath::Clamp((EndFrame * PixelsPerFrame), 0, UINT32_MAX));
		FadeRegions[Index].FadeInEndX = static_cast<uint32>(FMath::Clamp(StrongFade->GetFadeRegions()[Index].FadeIn->Duration * TransformationWaveInfo.SampleRate * PixelsPerFrame + FadeRegions[Index].FadeInStartX, 0, UINT32_MAX));
		FadeRegions[Index].FadeOutStartX = static_cast<uint32>(FMath::Clamp(FadeRegions[Index].FadeOutEndX - (StrongFade->GetFadeRegions()[Index].FadeOut->Duration * TransformationWaveInfo.SampleRate * PixelsPerFrame), 0, UINT32_MAX));
	}

	FVector2D MouseAbsolutePosition = UWidgetLayoutLibrary::GetMousePositionOnPlatform();
	MousePosition = AllottedGeometry.AbsoluteToLocal(MouseAbsolutePosition);

	GenerateFadeCurves(AllottedGeometry);
	UpdateInteractionRange(AllottedGeometry);
}

FReply FWaveformTransformationTrimFadeRenderer::OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	check(StrongTrimFade);
	FReply Reply = FWaveformTransformationFadeRenderer::OnMouseButtonDown(OwnerWidget, MyGeometry, MouseEvent);

	if (FadeInteractionTypeAndIndex.Key != EFadeInteractionType::None)
	{
		StrongTrimFade->Modify();
	}

	return Reply;
}

FReply FWaveformTransformationTrimFadeRenderer::OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	check(StrongTrimFade);

	if (FadeInteractionTypeAndIndex.Key != EFadeInteractionType::None)
	{
		SetPropertyValueDependingOnInteractionType(MouseEvent, MyGeometry);

		if (TransactionResult != INDEX_NONE)
		{
			EndTransaction();
			TransactionResult = INDEX_NONE;
		}

		FadeInteractionTypeAndIndex.Key = EFadeInteractionType::None;
		FadeInteractionTypeAndIndex.Value = 0;

		return FReply::Handled().ReleaseMouseCapture();
	}

	return FReply::Unhandled();
}

FReply FWaveformTransformationTrimFadeRenderer::OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	check(StrongTrimFade);
	if (MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton) && FadeInteractionTypeAndIndex.Key != EFadeInteractionType::None)
	{
		SetPropertyValueDependingOnInteractionType(MouseEvent, MyGeometry, EPropertyChangeType::Interactive);

		return FReply::Handled().CaptureMouse(OwnerWidget.AsShared());
	}

	return FReply::Unhandled();
}

void FWaveformTransformationTrimFadeRenderer::SetWaveformTransformation(TObjectPtr<UWaveformTransformationBase> InTransformation)
{
	StrongTrimFade = TStrongObjectPtr<UWaveformTransformationTrimFade>(CastChecked<UWaveformTransformationTrimFade>(InTransformation.Get()));
	check(StrongTrimFade->FadeTransformation);

	FWaveformTransformationFadeRenderer::SetWaveformTransformation(StrongTrimFade->FadeTransformation);

	TWeakObjectPtr<UWaveformTransformationTrimFade> InWeakTrimFade = StrongTrimFade.Get();
	OnFadeModeChanged.BindLambda([InWeakTrimFade]()
		{
			if (TStrongObjectPtr<UWaveformTransformationTrimFade> PinnedTrimFade = InWeakTrimFade.Pin())
			{
				PinnedTrimFade->PostEditChange();
			}
		});
}

void FWaveformTransformationTrimFadeRenderer::SetPropertyValueDependingOnInteractionType(const FPointerEvent& MouseEvent, const FGeometry& WidgetGeometry, const EPropertyChangeType::Type ChangeType)
{
	check(StrongFade);
	check(StrongTrimFade);
	check(PixelsPerFrame > 0.0);
	check(StrongFade->GetSampleRate() > 0);
	check(WidgetGeometry.GetLocalSize().X > 0.f);
	const FVector2D LocalCursorPosition = GetLocalCursorPosition(MouseEvent, WidgetGeometry);
	const float LocalCursorXRatio = FMath::Clamp(LocalCursorPosition.X / WidgetGeometry.GetLocalSize().X, 0.f, 1.f);
	const double SelectedTime = ConvertXRatioToTime(LocalCursorXRatio);
	const float NumFrames = TransformationWaveInfo.TotalNumSamples / TransformationWaveInfo.NumChannels;
	const float MaxDuration = NumFrames / StrongFade->GetSampleRate();

	if (StrongFade->GetFadeRegions().IsValidIndex(FadeInteractionTypeAndIndex.Value))
	{
		check(StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeIn);
		check(StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeOut);

		FProperty* Property = nullptr;

		switch (FadeInteractionTypeAndIndex.Key)
		{
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::None:
			break;
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeInStart:
		{
			StrongTrimFade->StartTime = FMath::Min(SelectedTime, StrongTrimFade->EndTime);
			Property = UWaveformTransformationTrimFade::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UWaveformTransformationTrimFade, StartTime));
			
			break;
		}
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeInEnd:
		{
			float StartFadeDurationValue = 0;
			StartFadeDurationValue = FMath::Clamp(SelectedTime - (FadeRegions[FadeInteractionTypeAndIndex.Value].FadeInStartX / PixelsPerFrame) / StrongFade->GetSampleRate(), 0.f, MaxDuration);
			StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeIn->Duration = StartFadeDurationValue;
			Property = UWaveformTransformationTrimFade::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UWaveformTransformationTrimFade, FadeTransformation));

			break;
		}
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeOutStart:
		{
			float EndFadeDurationValue = 0;
			const float diff = ((FadeRegions[FadeInteractionTypeAndIndex.Value].FadeOutEndX / PixelsPerFrame) / StrongFade->GetSampleRate()) - SelectedTime;
			EndFadeDurationValue = FMath::Clamp(diff, 0.f, MaxDuration);
			StrongFade->GetFadeRegions()[FadeInteractionTypeAndIndex.Value].FadeOut->Duration = EndFadeDurationValue;
			Property = UWaveformTransformationTrimFade::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UWaveformTransformationTrimFade, FadeTransformation));

			break;
		}
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::ScrubbingFadeOutEnd:
		{
			StrongTrimFade->EndTime = FMath::Max(SelectedTime, StrongTrimFade->StartTime);
			Property = UWaveformTransformationTrimFade::StaticClass()->FindPropertyByName(GET_MEMBER_NAME_CHECKED(UWaveformTransformationTrimFade, EndTime));

			break;
		}
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::RightClickFadeIn:
		case FWaveformTransformationFadeRenderer::EFadeInteractionType::RightClickFadeOut:
			ShowSelectFadeModeMenuAtCursor(WidgetGeometry, MouseEvent);
			break;
		default:
			break;
		}

		if (Property)
		{
			FPropertyChangedEvent PropertyChangedEvent(Property, ChangeType);
			StrongTrimFade->PostEditChangeProperty(PropertyChangedEvent);
		}
	}
}
