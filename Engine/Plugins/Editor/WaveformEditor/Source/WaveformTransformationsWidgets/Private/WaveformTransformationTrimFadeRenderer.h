// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WaveformTransformationRendererBase.h"

#include "WaveformTransformationFadeRenderer.h"
#include "WaveformTransformationTrimFade.h"


class FWaveformTransformationTrimFadeRenderer : public FWaveformTransformationFadeRenderer
{
public:

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	virtual void SetWaveformTransformation(TObjectPtr<UWaveformTransformationBase> InTransformation) override;

private:

	void SetPropertyValueDependingOnInteractionType(const FPointerEvent& MouseEvent, const FGeometry& WidgetGeometry, const EPropertyChangeType::Type ChangeType = EPropertyChangeType::ValueSet);

	TStrongObjectPtr<UWaveformTransformationTrimFade> StrongTrimFade = nullptr;
};
