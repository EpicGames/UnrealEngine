// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationsWidgetsModule.h"

#include "Modules/ModuleManager.h"
#include "WaveformTransformationLog.h"
#include "WaveformTransformationFade.h"
#include "WaveformTransformationFadeRenderer.h"
#include "WaveformTransformationMarkers.h"
#include "WaveformTransformationMarkersRenderer.h"
#include "WaveformTransformationRendererMapper.h"
#include "WaveformTransformationTrimFade.h"
#include "WaveformTransformationTrimFadeRenderer.h"

void FWaveformTransformationsWidgetsModule::StartupModule()
{
	FWaveformTransformationRendererMapper& RendererMapper = FWaveformTransformationRendererMapper::Get();
	RendererMapper.RegisterRenderer<FWaveformTransformationFadeRenderer>(UWaveformTransformationFade::StaticClass());
	RendererMapper.RegisterRenderer<FWaveformTransformationTrimFadeRenderer>(UWaveformTransformationTrimFade::StaticClass());
	RendererMapper.RegisterRenderer<FWaveformTransformationMarkerRenderer>(UWaveformTransformationMarkers::StaticClass()); 
}

void FWaveformTransformationsWidgetsModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FWaveformTransformationsWidgetsModule, WaveformTransformationsWidgets)