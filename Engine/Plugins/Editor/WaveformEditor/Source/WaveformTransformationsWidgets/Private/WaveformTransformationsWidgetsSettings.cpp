// Copyright Epic Games, Inc. All Rights Reserved.

#include "WaveformTransformationsWidgetsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WaveformTransformationsWidgetsSettings)

UWaveformTransformationsWidgetsSettings::UWaveformTransformationsWidgetsSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, MarkerColor(WaveformTransformationWidgetSharedDefaults::DefaultMarkerColor)
	, LoopColors(WaveformTransformationWidgetSharedDefaults::DefaultLoopColors)
	, LabelTextColor(WaveformTransformationWidgetSharedDefaults::DefaultLabelTextColor)
	, LabelFontSize(WaveformTransformationWidgetSharedDefaults::DefaultLabelFontSize)
{
}

FName UWaveformTransformationsWidgetsSettings::GetCategoryName() const
{
	return TEXT("Plugins");
}

FName UWaveformTransformationsWidgetsSettings::GetSectionName() const
{
	return TEXT("Waveform Transformations Display");
}

#if WITH_EDITOR
FText UWaveformTransformationsWidgetsSettings::GetSectionText() const
{
	return NSLOCTEXT("WaveformTransformationsDisplay", "WaveformTransformationsDisplaySettingsSection", "Waveform Transformations Display");
}

void UWaveformTransformationsWidgetsSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.Property != nullptr)
	{
		SettingsChangedDelegate.Broadcast(PropertyChangedEvent.GetPropertyName(), this);
	}
}

void UWaveformTransformationsWidgetsSettings::PostEditUndo()
{
	Super::PostEditUndo();

	SettingsChangedDelegate.Broadcast(NAME_None, this);
}

FOnWaveformTransformationsWidgetsSettingsChanged& UWaveformTransformationsWidgetsSettings::OnSettingChanged()
{
	return SettingsChangedDelegate;
}

FOnWaveformTransformationsWidgetsSettingsChanged UWaveformTransformationsWidgetsSettings::SettingsChangedDelegate;
#endif// WITH_EDITOR
