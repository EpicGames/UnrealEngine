// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"

#include "WaveformTransformationsWidgetsSettings.generated.h"

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnWaveformTransformationsWidgetsSettingsChanged, const FName& /*Property Name*/, const UWaveformTransformationsWidgetsSettings*);

namespace WaveformTransformationWidgetSharedDefaults
{
	const FLinearColor DefaultMarkerColor			= FLinearColor(1.f, 0.9f, 0, 0.5f);
	const FLinearColor DefaultSelectedMarkerColor = FLinearColor(1.f, 0.9f, 0, 1.f);
	const TArray<FLinearColor> DefaultLoopColors	= { FLinearColor(0.8f, 0.6f, 1.f, 0.75f), FLinearColor(0.2f, 0.7f, 1.f, 0.75f) };
	const FLinearColor DefaultLabelTextColor		= FLinearColor(0.9f, 0.9f, 0.9f, 1.f);
	const float DefaultLabelFontSize				= 10.f;
}

UCLASS(config = EditorPerProjectUserSettings, defaultconfig, meta = (DisplayName = "Waveform Transformations Display"))
class UWaveformTransformationsWidgetsSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:

	UWaveformTransformationsWidgetsSettings(const FObjectInitializer& ObjectInitializer);

	//~ Begin UDeveloperSettings interface
	virtual FName GetCategoryName() const;

	virtual FName GetSectionName() const override;

#if WITH_EDITOR
	virtual FText GetSectionText() const override;

	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostEditUndo() override;
	//~ End UDeveloperSettings interface

	static FOnWaveformTransformationsWidgetsSettingsChanged& OnSettingChanged();
#endif


	//Set the color of your Waveform Markers
	UPROPERTY(config, EditAnywhere, Category = "Markers")
	FLinearColor MarkerColor;

	//A list of colors so marker regions can have different colors
	UPROPERTY(config, EditAnywhere, Category = "Markers")
	TArray<FLinearColor> LoopColors;

	UPROPERTY(config, EditAnywhere, Category = "Markers")
	FLinearColor LabelTextColor;

	UPROPERTY(config, EditAnywhere, Category = "Markers", Meta = (ClampMin = "1", ClampMax = "15"))
	float LabelFontSize;


private: 
	static FOnWaveformTransformationsWidgetsSettingsChanged SettingsChangedDelegate;
};