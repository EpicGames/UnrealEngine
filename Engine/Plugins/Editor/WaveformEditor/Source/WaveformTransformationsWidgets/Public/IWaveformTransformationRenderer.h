// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWaveformTransformation.h"
#include "Templates/Function.h"

class IPropertyHandle;
class FCursorReply;
class FPaintArgs;
class FReply;
class FSlateRect;
class FSlateWindowElementList;
class FWidgetStyle;
struct FKeyEvent;
class SWidget;
class UClass;
struct FGeometry;
struct FPointerEvent;

struct FWaveformTransformationRenderInfo
{
	float SampleRate = 0.f;
	int32 NumChannels = 0;
	int64 StartFrameOffset = 0;	//the first frame this transformation can act upon
	uint32 TotalNumSamples = 0;	//the number of total samples in the original waveform
	uint32 NumSamplesAvailable = 0;	//the number of samples available to the transformation
};

class IWaveformTransformationRenderer
{
public:
	virtual ~IWaveformTransformationRenderer() = default;

	/* SWidget interceptor methods */
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const = 0;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) = 0;
	virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) = 0;
	virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) = 0;
	virtual FReply OnMouseButtonDoubleClick(SWidget& OwnerWidget, const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) = 0;
	virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) = 0;
	virtual FReply OnMouseWheel(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) = 0;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) = 0;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) = 0;
	virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const = 0;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) = 0;

	/* Gets info about the transformation being rendered */
	virtual void SetTransformationWaveInfo(const FWaveformTransformationRenderInfo& InWaveInfo) = 0;
	
	virtual void SetWaveformTransformation(TObjectPtr<UWaveformTransformationBase> InTransformation) = 0;

	UE_DEPRECATED(5.7, "PropertyHandles have been deprecated.")
	/* Gets the transformations properties. IPropertyHandle is used to get/set properties to have full support of change notifiers, editor undo ecc  */
	virtual void SetPropertyHandles(const TArray<TSharedRef<IPropertyHandle>>& InPropertyHandles) = 0;
};


