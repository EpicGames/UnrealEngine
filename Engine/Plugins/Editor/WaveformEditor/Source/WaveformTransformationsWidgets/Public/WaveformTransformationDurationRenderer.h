// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Geometry.h"
#include "Rendering/DrawElements.h"
#include "Styling/AppStyle.h"
#include "WaveformTransformationRendererBase.h"

#define UE_API WAVEFORMTRANSFORMATIONSWIDGETS_API


namespace WaveformTransformationDurationHiglightParams
{
	const FLazyName BackgroundBrushName = TEXT("WhiteBrush");
	const FLinearColor BoxColor = FLinearColor(0.f, 0.f, 0.f, 0.7f);
}

class FWaveformTransformationDurationRenderer : public FWaveformTransformationRendererBase
{
public:
	UE_API explicit FWaveformTransformationDurationRenderer(const uint32 InOriginalWaveformNumFrames);

	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	UE_API void SetOriginalWaveformFrames(const uint32 NumFrames);

private:
	uint32 OriginalWaveformNumFrames = 0;
};

#undef UE_API
