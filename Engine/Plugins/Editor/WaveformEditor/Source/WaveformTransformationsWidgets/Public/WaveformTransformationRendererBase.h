// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IDetailCustomization.h"
#include "IWaveformTransformationRenderer.h"
#include "PropertyHandle.h"

#define UE_API WAVEFORMTRANSFORMATIONSWIDGETS_API

class FWaveformTransformationRendererBase : public IWaveformTransformationRenderer
{
public:
	FWaveformTransformationRendererBase() = default;
	
	UE_API virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override {};
	UE_API virtual FReply OnMouseButtonDown(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonUp(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseButtonDoubleClick(SWidget& OwnerWidget, const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;
	UE_API virtual FReply OnMouseMove(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	UE_API virtual FReply OnMouseWheel(SWidget& OwnerWidget, const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override {};
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override {};
	UE_API virtual FCursorReply OnCursorQuery(const FGeometry& MyGeometry, const FPointerEvent& CursorEvent) const override;
	UE_API virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	UE_API virtual void SetTransformationWaveInfo(const FWaveformTransformationRenderInfo& InWaveInfo) override;
	virtual void SetWaveformTransformation(TObjectPtr<UWaveformTransformationBase> InTransformation) override {}

	UE_DEPRECATED(5.7, "PropertyHandles have been deprecated.")
	UE_API virtual void SetPropertyHandles(const TArray<TSharedRef<IPropertyHandle>>& InPropertyHandles) override;

protected:
	UE_DEPRECATED(5.7, "PropertyHandles have been deprecated.")
	UE_API TSharedPtr<IPropertyHandle> GetPropertyHandle(const FName& PropertyName) const;

	template<typename T>
	UE_DEPRECATED(5.7, "PropertyHandles have been deprecated.")
	T GetPropertyValue(const TSharedPtr<IPropertyHandle>& PropertyHandle)
	{
		check(PropertyHandle.IsValid());
		T OutValue;
		PropertyHandle->GetValue(OutValue);
		return OutValue;
	}

	template<typename T>
	UE_DEPRECATED(5.7, "PropertyHandles have been deprecated.")
	void SetPropertyValue(const FName& PropertyName, const T Value, const EPropertyValueSetFlags::Type Flags = EPropertyValueSetFlags::DefaultFlags)
	{
		TSharedPtr<IPropertyHandle> Handle = GetPropertyHandle(PropertyName);
		Handle->SetValue(Value, Flags);
	}

	UE_API int32 BeginTransaction(const TCHAR* TransactionContext, const FText& Description, UObject* PrimaryObject);
	UE_API int32 EndTransaction();

	FWaveformTransformationRenderInfo TransformationWaveInfo;

	static constexpr float InteractionPixelXDelta = 10;
	static constexpr float InteractionRatioYDelta = 0.07f;
	static constexpr float MouseWheelStep = 0.1f;

private:
	UE_DEPRECATED(5.7, "PropertyHandles have been deprecated.")
	TArray<TSharedRef<IPropertyHandle>> CachedPropertyHandles;
};

#undef UE_API
