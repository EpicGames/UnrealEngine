// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IWaveformTransformationRenderer.h"

#define UE_API WAVEFORMTRANSFORMATIONSWIDGETS_API

using FWaveformTransformRendererInstantiator = TFunction<TSharedPtr<IWaveformTransformationRenderer>()>;

class FWaveformTransformationRendererMapper
{
public:
	/** Access the singleton instance for mapper */
	static UE_API FWaveformTransformationRendererMapper& Get();
	static UE_API void Init();

	template<typename ConcreteRendererType>
	bool RegisterRenderer(const UClass* SupportedTransformation)
	{
		FWaveformTransformRendererInstantiator Instantiator = []()
		{
			return MakeShared<ConcreteRendererType>();
		};

		check(Instance)
		Instance->TransformationsTypeMap.Add(SupportedTransformation, Instantiator);
		return true;
	}

	UE_API void UnregisterRenderer(const UClass* SupportedTransformation);

	UE_API FWaveformTransformRendererInstantiator* GetRenderer(const UClass* WaveformTransformationClass);

private:
	static UE_API TUniquePtr<FWaveformTransformationRendererMapper> Instance;

	TMap<const UClass*, FWaveformTransformRendererInstantiator> TransformationsTypeMap;
};

#undef UE_API
