// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class WaveformTransformationsWidgets : ModuleRules
	{
		public WaveformTransformationsWidgets(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"AudioExtensions",
					"AudioWidgets",
					"ContentBrowser",
					"Core",
					"CoreUObject",
					"DeveloperSettings",
					"EditorStyle",
					"Engine",
					"InputCore",
					"PropertyEditor",
					"Slate",
					"SignalProcessing",
					"SlateCore", 
					"ToolMenus", 
					"UnrealEd",
					"UMG",
					"WaveformTransformations",
				}
			);
		}
	}
}