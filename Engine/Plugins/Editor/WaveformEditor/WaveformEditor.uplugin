{
	"FileVersion": 3,
	"Version": 0.1,
	"VersionName": "0.1",
	"FriendlyName": "Waveform Editor",
	"Description": "Editor tool for waveforms",
	"Category": "Audio",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "WaveformEditor",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "WaveformEditorWidgets",
			"Type": "Editor",
			"LoadingPhase": "Default"
		},
		{
			"Name": "WaveformTransformations",
			"Type": "Editor",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "WaveformTransformationsWidgets",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "AudioSynesthesia",
			"Enabled": true
		},
		{
			"Name": "AudioWidgets",
			"Enabled" :  true
		},
		{
			"Name": "EditorScriptingUtilities",
			"Enabled": true
		}
	]
}